{
	"FileVersion" : 1,
	"Version" : 1,
	"VersionName" : "0.1",
	"FriendlyName" : "Sequence Validator",
	"Description" : "A tool that provides validation rules for detecting common errors in sequences",
	"Category" : "Editor",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com/",
	"DocsURL" : "",
	"MarketplaceURL" : "",
	"SupportURL" : "",
	"EnabledByDefault" : true,
	"CanContainContent" : false,
	"IsExperimentalVersion" : true,
	"Installed" : false,
	"Modules" :
	[
		{
			"Name" : "SequenceValidator",
			"Type" : "Editor",
			"LoadingPhase" : "PostEngineInit"
		}
	],
	"Plugins":
	[
		{
			"Name": "LevelSequenceEditor",
			"Enabled": true
		}
	]
}
