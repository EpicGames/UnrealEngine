// Copyright Epic Games, Inc. All Rights Reserved.

#include "SequenceValidatorCommands.h"

#include "Framework/Commands/Commands.h"
#include "Framework/Commands/InputChord.h"
#include "Framework/Commands/UICommandInfo.h"
#include "SequenceValidatorStyle.h"

#define LOCTEXT_NAMESPACE "SequenceValidatorCommands"

namespace UE::Sequencer
{

FSequenceValidatorCommands::FSequenceValidatorCommands()
	: TCommands<FSequenceValidatorCommands>(
			"SequenceValidator",
			LOCTEXT("SequenceValidator", "Sequence Validator"),
			NAME_None,
			FSequenceValidatorStyle::Get()->GetStyleSetName()
		)
{
}

void FSequenceValidatorCommands::RegisterCommands()
{
	UI_COMMAND(
			StartValidation, "Start Validation", 
			"Start validating the sequences in the queue",
			EUserInterfaceActionType::Button,
			FInputChord());
}

}  // namespace UE::Sequencer

#undef LOCTEXT_NAMESPACE

