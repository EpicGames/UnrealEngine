// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Framework/Commands/Commands.h"

class FUICommandInfo;
class UClass;
struct FInputChord;

namespace UE::Sequencer
{

class FSequenceValidatorCommands : public TCommands<FSequenceValidatorCommands>
{
public:

	FSequenceValidatorCommands();

	virtual void RegisterCommands() override;
	
public:

	TSharedPtr<FUICommandInfo> StartValidation;
};

}  // namespace UE::Sequencer

