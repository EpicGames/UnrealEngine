// Copyright Epic Games, Inc. All Rights Reserved.

#include "ISequenceValidatorModule.h"

#include "Editor.h"
#include "SequenceValidatorCommands.h"
#include "Validation/Rules/SequenceValidationRule_DuplicateKeys.h"
#include "Validation/Rules/SequenceValidationRule_SectionAlignments.h"
#include "Validation/Rules/SequenceValidationRule_UnassignedBindingsAndAssets.h"
#include "Validation/Rules/SequenceValidationRule_WholeSectionRanges.h"
#include "Widgets/SSequenceValidator.h"

DEFINE_LOG_CATEGORY(LogSequenceValidator);

namespace UE::Sequencer
{

class FSequenceValidatorModule
	: public ISequenceValidatorModule
{

protected:

	// IModuleInterface interface.
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

	// ISequenceValidatorModule interface.
	virtual FSequenceValidationRuleID RegisterValidationRule(FSequenceValidationRuleInfo&& InRuleInfo) override;
	virtual TConstArrayView<FSequenceValidationRuleInfo> GetValidationRules() const override;
	virtual void UnregisterValidationRule(FSequenceValidationRuleID InHandle) override;

private:

	void OnPostEngineInit();

	void RegisterBuiltInRules();
	void UnregisterBuiltInRules();

private:

	TArray<FSequenceValidationRuleInfo> ValidationRules;
	TArray<FSequenceValidationRuleID> BuiltInValidationRules;
};

void FSequenceValidatorModule::StartupModule()
{
	if (GEditor)
	{
		OnPostEngineInit();
	}
	else
	{
		FCoreDelegates::GetOnPostEngineInit().AddRaw(this, &FSequenceValidatorModule::OnPostEngineInit);
	}

	FSequenceValidatorCommands::Register();
	
	RegisterBuiltInRules();
}

void FSequenceValidatorModule::ShutdownModule()
{
	SSequenceValidator::UnregisterTabSpawners();

	FSequenceValidatorCommands::Unregister();

	UnregisterBuiltInRules();

	FCoreDelegates::GetOnPostEngineInit().RemoveAll(this);
}

void FSequenceValidatorModule::OnPostEngineInit()
{
	SSequenceValidator::RegisterTabSpawners();
}

void FSequenceValidatorModule::RegisterBuiltInRules()
{
	BuiltInValidationRules.Add(
			RegisterValidationRule(FSequenceValidationRule_SectionAlignments::MakeRuleInfo()));
	BuiltInValidationRules.Add(
		RegisterValidationRule(FSequenceValidationRule_UnassignedBindingsAndAssets::MakeRuleInfo()));
	BuiltInValidationRules.Add(
		RegisterValidationRule(FSequenceValidationRule_WholeSectionRanges::MakeRuleInfo()));
	BuiltInValidationRules.Add(
		RegisterValidationRule(FSequenceValidationRule_DuplicateKeys::MakeRuleInfo()));
}

void FSequenceValidatorModule::UnregisterBuiltInRules()
{
	for (FDelegateHandle Handle : BuiltInValidationRules)
	{
		UnregisterValidationRule(Handle);
	}
	BuiltInValidationRules.Reset();
}

FSequenceValidationRuleID FSequenceValidatorModule::RegisterValidationRule(FSequenceValidationRuleInfo&& InRuleInfo)
{
	ValidationRules.Add(MoveTemp(InRuleInfo));
	return ValidationRules.Last().RuleFactory.GetHandle();
}

TConstArrayView<FSequenceValidationRuleInfo> FSequenceValidatorModule::GetValidationRules() const
{
	return ValidationRules;
}

void FSequenceValidatorModule::UnregisterValidationRule(FDelegateHandle InHandle)
{
	ValidationRules.RemoveAll(
			[=](const FSequenceValidationRuleInfo& RuleInfo) 
			{
				return RuleInfo.RuleFactory.GetHandle() == InHandle; 
			});
}

}  // namespace UE::Sequencer

IMPLEMENT_MODULE(UE::Sequencer::FSequenceValidatorModule, SequenceValidator);

