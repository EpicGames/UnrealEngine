// Copyright Epic Games, Inc. All Rights Reserved.

#include "SequenceValidatorStyle.h"

#include "Interfaces/IPluginManager.h"
#include "Misc/Paths.h"
#include "Styling/AppStyle.h"
#include "Styling/CoreStyle.h"
#include "Styling/SlateStyleRegistry.h"
#include "Styling/SlateStyleMacros.h"
#include "Styling/SlateTypes.h"

namespace UE::Sequencer
{

TSharedPtr<FSequenceValidatorStyle> FSequenceValidatorStyle::Singleton;

FSequenceValidatorStyle::FSequenceValidatorStyle()
	: FSlateStyleSet("SequenceValidatorStyle")
{
	const FVector2D Icon16x16(16.0f, 16.0f);

	const FString ContentDir = IPluginManager::Get().FindPlugin(TEXT("SequenceValidator"))->GetContentDir();
	SetContentRoot(ContentDir);
	SetCoreContentRoot(FPaths::EngineContentDir() / TEXT("Slate"));

	const ISlateStyle& CoreStyle = FAppStyle::Get();
	const FTextBlockStyle NormalText = CoreStyle.GetWidgetStyle<FTextBlockStyle>("NormalText");
	const FTableRowStyle AlternatingTableRowStyle = CoreStyle.GetWidgetStyle<FTableRowStyle>("TableView.AlternatingRow");

	// Validation rules icons.
	Set("ValidationRule.RowStyle", AlternatingTableRowStyle);
	Set("ValidationRule.Rule", new IMAGE_BRUSH("Icons/ValidationRule-Rule", Icon16x16));

	// Validation results icons.
	Set("ValidationResult.Key", new IMAGE_BRUSH("Icons/ValidationResult-Key", Icon16x16));
	Set("ValidationResult.Section", new IMAGE_BRUSH("Icons/ValidationResult-Section", Icon16x16));
	Set("ValidationResult.Track", new IMAGE_BRUSH("Icons/ValidationResult-Track", Icon16x16));
	Set("ValidationResult.Sequence", new IMAGE_BRUSH("Icons/ValidationResult-Sequence", Icon16x16));
	Set("ValidationResult.Pass", new IMAGE_BRUSH_SVG("Icons/ValidationResult-Pass", Icon16x16));

	Set("ValidationResult.RowStyle", AlternatingTableRowStyle);

	// Command icons.
	Set("SequenceValidator.QueueSequence", new CORE_IMAGE_BRUSH_SVG("Starship/Common/plus", Icon16x16));
	Set("SequenceValidator.StartValidation", new IMAGE_BRUSH_SVG("Icons/SequenceValidation", Icon16x16));

	// Tab icons.
	Set("SequenceValidator.TabIcon", new IMAGE_BRUSH_SVG("Icons/SequenceValidation", Icon16x16));

	// Title icons.
	Set("SequenceValidator.QueueTitleIcon", new IMAGE_BRUSH_SVG("Icons/ValidationQueue", Icon16x16));
	Set("SequenceValidator.RulesTitleIcon", new IMAGE_BRUSH_SVG("Icons/ValidationRule", Icon16x16));
	Set("SequenceValidator.ResultsTitleIcon", new IMAGE_BRUSH_SVG("Icons/ValidationResults", Icon16x16));

	// Text styles.

	Set("SequenceValidator.PanelTitleText", FTextBlockStyle(NormalText)
		.SetFont(DEFAULT_FONT("Regular", 16))
		.SetColorAndOpacity(FLinearColor(0.8f, 0.8f, 0.8f))
		.SetHighlightColor(FLinearColor(1.0f, 1.0f, 1.0f))
		.SetShadowOffset(FVector2f::UnitVector)
		.SetShadowColorAndOpacity(FLinearColor(0, 0, 0, 0.9f)));

	Set("ValidatorRule.RuleNameText", FTextBlockStyle(NormalText)
		.SetFont(DEFAULT_FONT("Bold", 12)));

	Set("ValidatorRule.RuleDescriptionText", FTextBlockStyle(NormalText)
		.SetFont(DEFAULT_FONT("Regular", 10)));

	FSlateStyleRegistry::RegisterSlateStyle(*this);
}

FSequenceValidatorStyle::~FSequenceValidatorStyle()
{
	FSlateStyleRegistry::UnRegisterSlateStyle(*this);
}

TSharedRef<FSequenceValidatorStyle> FSequenceValidatorStyle::Get()
{
	if (!Singleton.IsValid())
	{
		Singleton = MakeShareable(new FSequenceValidatorStyle);
	}
	return Singleton.ToSharedRef();
}

}  // namespace UE::Sequencer

