// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/SlateStyle.h"

namespace UE::Sequencer
{

/**
 * Implements the visual style of the sequence validator.
 */
class FSequenceValidatorStyle final : public FSlateStyleSet
{
public:
	
	FSequenceValidatorStyle();
	virtual ~FSequenceValidatorStyle();

	static TSharedRef<FSequenceValidatorStyle> Get();

private:

	static TSharedPtr<FSequenceValidatorStyle> Singleton;
};

}  // namespace UE::Sequencer

