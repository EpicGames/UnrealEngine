// Copyright Epic Games, Inc. All Rights Reserved.

#include "Validation/Rules/SequenceValidationRule_DuplicateKeys.h"

#include "Channels/MovieSceneChannel.h"
#include "Channels/MovieSceneChannelProxy.h"
#include "Internationalization/Internationalization.h"
#include "MovieScene.h"
#include "MovieSceneCommonHelpers.h"
#include "MovieSceneSection.h"
#include "MovieSceneSequence.h"
#include "MovieSceneSequenceVisitor.h"
#include "Validation/SequenceValidationResult.h"

#define LOCTEXT_NAMESPACE "SequenceValidationRule_DuplicateKeys"

namespace UE::Sequencer
{

class FDuplicateKeysVisitor : public UE::MovieScene::ISequenceVisitor
{
	using FSubSequenceSpace = UE::MovieScene::FSubSequenceSpace;

public:

	FDuplicateKeysVisitor(const FSequenceValidationRuleInfo& InRuleInfo, FSequenceValidationResults& OutResults)
		: RuleInfo(InRuleInfo)
		, Results(OutResults)
	{
	}

	virtual void VisitStartSubSequence(UMovieSceneSubSection* SubSection, const FGuid& ObjectBindingID, const FSubSequenceSpace& LocalSpace) override
	{
		CurrentSubSectionTrail.Add(SubSection);
	}

	virtual void VisitEndSubSequence(UMovieSceneSubSection* SubSection, const FGuid& ObjectBindingID, const FSubSequenceSpace& LocalSpace) override
	{
		ensure(CurrentSubSectionTrail.Num() > 0 && CurrentSubSectionTrail.Last() == SubSection);
		CurrentSubSectionTrail.Pop(EAllowShrinking::No);
	}

	virtual void VisitSection(UMovieSceneTrack* Track, UMovieSceneSection* Section, const FGuid& ObjectBindingID, const FSubSequenceSpace& LocalSpace) override
	{
		for (const FMovieSceneChannelEntry& ChannelEntry : Section->GetChannelProxy().GetAllEntries())
		{
			TConstArrayView<FMovieSceneChannel*> Channels = ChannelEntry.GetChannels();
			TConstArrayView<FMovieSceneChannelMetaData> ChannelMetaData = ChannelEntry.GetMetaData();

			for (int32 ChannelIndex = 0; ChannelIndex < Channels.Num(); ++ChannelIndex)
			{
				if (FMovieSceneChannel* Channel = Channels[ChannelIndex])
				{
					TArray<FFrameNumber> KeyTimes;
					TArray<FKeyHandle> KeyHandles;

					Channel->GetKeys(TRange<FFrameNumber>::All(), &KeyTimes, &KeyHandles);

					TSet<FFrameNumber> UniqueKeyTimes;
					TArray<FKeyHandle> DuplicateKeyHandles;
					FFrameNumber LocalTime;

					for (int32 KeyIndex = 0; KeyIndex < KeyTimes.Num(); ++KeyIndex)
					{
						if (!UniqueKeyTimes.Contains(KeyTimes[KeyIndex]))
						{
							UniqueKeyTimes.Add(KeyTimes[KeyIndex]);
						}
						else
						{
							// Found duplicate!
							DuplicateKeyHandles.Add(KeyHandles[KeyIndex]);
							LocalTime = KeyTimes[KeyIndex];
						}
					}

					if (DuplicateKeyHandles.Num() > 0)
					{
						const FMovieSceneChannelMetaData& MetaData = ChannelMetaData[ChannelIndex];
						AddWarning(Section, Channel, MetaData, DuplicateKeyHandles, LocalTime);
					}
				}
			}
		}
	}

private:

	void AddWarning(UMovieSceneSection* Section, FMovieSceneChannel* Channel, const FMovieSceneChannelMetaData& ChannelMetaData, const TArray<FKeyHandle>& KeyHandles, FFrameTime LocalTime)
	{
		TSharedRef<FSequenceValidationResult> NewResult = MakeShared<FSequenceValidationResult>(EMessageSeverity::Warning, Section, RuleInfo);
		NewResult->SetSubSectionTrail(CurrentSubSectionTrail);
		NewResult->SetTargetKeys(FTargetKeys(Channel, KeyHandles));
		NewResult->SetLocalTime(LocalTime);
		NewResult->SetUserMessage(FText::Format(LOCTEXT("DuplicateKeyWarningFmt", " Channel {0} : contains {1} duplicate key(s)"), MovieSceneHelpers::GetDisplayPathName(Section, Channel, ChannelMetaData), KeyHandles.Num()));
		Results.AddResult(NewResult);
	}

private:

	FSequenceValidationRuleInfo RuleInfo;
	TArray<UMovieSceneSubSection*> CurrentSubSectionTrail;
	FSequenceValidationResults& Results;
};

FSequenceValidationRuleInfo FSequenceValidationRule_DuplicateKeys::MakeRuleInfo()
{
	FSequenceValidationRuleInfo RuleInfo;
	RuleInfo.RuleName = LOCTEXT("RuleName", "Duplicate Keys");
	RuleInfo.RuleDescription = LOCTEXT("RuleDescription", "Check for duplicate keys on the same channel.");
	RuleInfo.RuleFactory = FOnCreateSequenceValidationRule::CreateLambda([]() { return MakeShared<FSequenceValidationRule_DuplicateKeys>(); });
	RuleInfo.RuleColor = FSequenceValidationRule::GetRuleColor(RuleInfo.RuleName);
	return RuleInfo;
}

void FSequenceValidationRule_DuplicateKeys::OnRun(const UMovieSceneSequence* InSequence, FSequenceValidationResults& OutResults) const
{
	UE::MovieScene::FSequenceVisitParams VisitParams;
	VisitParams.bVisitRootTracks = true;
	VisitParams.bVisitObjectBindings = true;
	VisitParams.bVisitSubSequences = true;
	VisitParams.bVisitTracks = true;
	VisitParams.bVisitSections = true;

	FDuplicateKeysVisitor Visitor(MakeRuleInfo(), OutResults);
	UE::MovieScene::VisitSequence(const_cast<UMovieSceneSequence*>(InSequence), VisitParams, Visitor);
}

}  // namespace UE::Sequencer

#undef LOCTEXT_NAMESPACE

