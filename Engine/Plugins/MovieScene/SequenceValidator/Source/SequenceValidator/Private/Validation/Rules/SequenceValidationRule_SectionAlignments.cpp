// Copyright Epic Games, Inc. All Rights Reserved.

#include "Validation/Rules/SequenceValidationRule_SectionAlignments.h"

#include "Algo/BinarySearch.h"
#include "Internationalization/Internationalization.h"
#include "MovieScene.h"
#include "MovieSceneCommonHelpers.h"
#include "MovieSceneTimeHelpers.h"
#include "MovieSceneSection.h"
#include "MovieSceneSequence.h"
#include "MovieSceneSequenceVisitor.h"
#include "MovieSceneTrack.h"
#include "Sections/MovieSceneSubSection.h"
#include "Tracks/MovieSceneCameraCutTrack.h"
#include "Tracks/MovieSceneCinematicShotTrack.h"
#include "Tracks/MovieSceneSubTrack.h"
#include "Validation/SequenceValidationResult.h"

#define LOCTEXT_NAMESPACE "SequenceValidationRule_SectionAlignments"

namespace UE::Sequencer
{

struct FImportantFrameTime
{
	FFrameTime FrameTime;
	FText Label;
};

bool operator< (const FImportantFrameTime& A, const FImportantFrameTime& B)
{
	return A.FrameTime < B.FrameTime;
}

bool operator== (const FImportantFrameTime& A, const FImportantFrameTime& B)
{
	return A.FrameTime == B.FrameTime;
}

uint32 GetTypeHash(const FImportantFrameTime& A)
{
	return HashCombine(::GetTypeHash(A.FrameTime.FrameNumber.Value), ::GetTypeHash(A.FrameTime.GetSubFrame()));
}

class FSectionAlignmentsImportantTimeVisitor : public UE::MovieScene::ISequenceVisitor
{
	using FSubSequenceSpace = UE::MovieScene::FSubSequenceSpace;

public:

	virtual void VisitTrack(UMovieSceneTrack* Track, const FGuid& ObjectBindingID, const FSubSequenceSpace& LocalSpace) override
	{
		if (UMovieSceneCameraCutTrack* CameraCutTrack = Cast<UMovieSceneCameraCutTrack>(Track))
		{
			for (UMovieSceneSection* CameraCutSection : CameraCutTrack->GetAllSections())
			{
				TRange<FFrameNumber> TimeRange = CameraCutSection->GetTrueRange();
				AddImportantTimes(
						TimeRange, LocalSpace,
						LOCTEXT("CameraCutBound", "Camera Cut Bound"),
						LOCTEXT("CameraCutBound", "Camera Cut Bound"));
			}
		}
	}

public:

	void AddImportantTimes(
			const TRange<FFrameNumber>& InTimeRange, const FSubSequenceSpace& LocalSpace,
			const FText& LowerTimeLabel, const FText& UpperTimeLabel)
	{
		FMovieSceneInverseSequenceTransform SequenceToRootTransform = LocalSpace.RootToSequenceTransform.Inverse();
		if (InTimeRange.HasLowerBound())
		{
			TOptional<FFrameTime> RootTime = SequenceToRootTransform.TryTransformTime(InTimeRange.GetLowerBoundValue());
			if (RootTime.IsSet())
			{
				ImportantTimes.FindOrAdd({ RootTime.GetValue(), LowerTimeLabel });
			}
		}
		if (InTimeRange.HasUpperBound())
		{
			TOptional<FFrameTime> RootTime = SequenceToRootTransform.TryTransformTime(InTimeRange.GetUpperBoundValue());
			if (RootTime.IsSet())
			{
				ImportantTimes.FindOrAdd({ RootTime.GetValue(), UpperTimeLabel });
			}
		}
	}

	void GetImportantTimes(TArray<FImportantFrameTime>& OutImportantTimes)
	{
		OutImportantTimes = ImportantTimes.Array();
	}

private:

	TSet<FImportantFrameTime> ImportantTimes;
};

struct FSectionAlignmentsValidationVisitor : public UE::MovieScene::ISequenceVisitor
{
	using FSubSequenceSpace = UE::MovieScene::FSubSequenceSpace;

public:

	FSectionAlignmentsValidationVisitor(const FSequenceValidationRuleInfo& InRuleInfo, TArrayView<FImportantFrameTime> InImportantTimes, FSequenceValidationResults& OutResults)
		: RuleInfo(InRuleInfo)
		, ImportantTimes(InImportantTimes)
		, Results(OutResults)
	{
	}

	void Initialize(const UMovieSceneSequence* InSequence)
	{
		UMovieScene* MovieScene = InSequence->GetMovieScene();

		ImportantTimes.Sort();
		CurrentSubSectionTrail.Reserve(4);

		// We will emit warnings for sections that start/end within 2 frames of an "important time".
		const FFrameRate& DisplayRate = MovieScene->GetDisplayRate();
		const FFrameRate& TickResolution = MovieScene->GetTickResolution();
		Threshold = FFrameRate::TransformTime(FFrameTime(2), DisplayRate, TickResolution);
	}

	virtual void VisitStartSubSequence(UMovieSceneSubSection* SubSection, const FGuid& ObjectBindingID, const FSubSequenceSpace& LocalSpace) override
	{
		CurrentSubSectionTrail.Add(SubSection);
	}

	virtual void VisitEndSubSequence(UMovieSceneSubSection* SubSection, const FGuid& ObjectBindingID, const FSubSequenceSpace& LocalSpace) override
	{
		ensure(CurrentSubSectionTrail.Num() > 0 && CurrentSubSectionTrail.Last() == SubSection);
		CurrentSubSectionTrail.Pop(EAllowShrinking::No);
	}

	virtual void VisitSection(UMovieSceneTrack* Track, UMovieSceneSection* Section, const FGuid& ObjectBindingID, const FSubSequenceSpace& LocalSpace) override
	{
		TRange<FFrameNumber> TimeRange = Section->GetTrueRange();
		FMovieSceneInverseSequenceTransform SequenceToRootTransform = LocalSpace.RootToSequenceTransform.Inverse();
		if (TimeRange.HasLowerBound())
		{
			MaybeWarnAbout(Results, SequenceToRootTransform, TimeRange.GetLowerBoundValue(), Section);
		}
		if (TimeRange.HasUpperBound())
		{
			MaybeWarnAbout(Results, SequenceToRootTransform, TimeRange.GetUpperBoundValue(), Section);
		}
	}

private:

	void MaybeWarnAbout(FSequenceValidationResults& OutResults, const FMovieSceneInverseSequenceTransform& SequenceToRootTransform, FFrameTime InTime, UMovieSceneSection* Section)
	{
		// Convert the local time to a root sequence time, so that we can compare it to all the important times that
		// are also in root time space.
		TOptional<FFrameTime> OptRootTime = SequenceToRootTransform.TryTransformTime(InTime);
		if (!OptRootTime.IsSet())
		{
			return;
		}

		const FFrameTime RootTime = OptRootTime.GetValue();

		const int32 NextTimeIndex = Algo::LowerBound(ImportantTimes, FImportantFrameTime{ RootTime });
		const int32 PreviousTimeIndex = NextTimeIndex - 1;

		bool bDoWarn = false;
		FText TimeLabel;
		if (ImportantTimes.IsValidIndex(PreviousTimeIndex))
		{
			const FImportantFrameTime& PreviousTime = ImportantTimes[PreviousTimeIndex];
			if (RootTime != PreviousTime.FrameTime && FMath::Abs((RootTime - PreviousTime.FrameTime)) <= Threshold)
			{
				bDoWarn = true;
				TimeLabel = PreviousTime.Label;
			}
		}
		if (ImportantTimes.IsValidIndex(NextTimeIndex))
		{
			const FImportantFrameTime& NextTime = ImportantTimes[NextTimeIndex];
			if (RootTime != NextTime.FrameTime && FMath::Abs((RootTime - NextTime.FrameTime)) <= Threshold)
			{
				bDoWarn = true;
				TimeLabel = NextTime.Label;
			}
		}

		if (bDoWarn)
		{
			TSharedRef<FSequenceValidationResult> NewResult = MakeShared<FSequenceValidationResult>(EMessageSeverity::Warning, Section, RuleInfo);
			NewResult->SetSubSectionTrail(CurrentSubSectionTrail);
			NewResult->SetLocalTime(InTime);
			NewResult->SetUserMessage(FText::Format(
				LOCTEXT("AlignmentWarningFmt", "Section {0} : bound is within two frames of {1}."),
						MovieSceneHelpers::GetDisplayPathName(Section), TimeLabel));
			OutResults.AddResult(NewResult);
		}
	}

private:

	FSequenceValidationRuleInfo RuleInfo;
	FFrameTime Threshold;
	TArray<FImportantFrameTime> ImportantTimes;
	TArray<UMovieSceneSubSection*> CurrentSubSectionTrail;
	FSequenceValidationResults& Results;
};

FSequenceValidationRuleInfo FSequenceValidationRule_SectionAlignments::MakeRuleInfo()
{
	FSequenceValidationRuleInfo RuleInfo;
	RuleInfo.RuleName = LOCTEXT("RuleName", "Section Alignments");
	RuleInfo.RuleDescription = LOCTEXT("RuleDescription", "Check that sections are not off by a couple frames from notable frames.");
	RuleInfo.RuleFactory = FOnCreateSequenceValidationRule::CreateLambda([]() { return MakeShared<FSequenceValidationRule_SectionAlignments>(); });
	RuleInfo.RuleColor = FSequenceValidationRule::GetRuleColor(RuleInfo.RuleName);
	return RuleInfo;
}

void FSequenceValidationRule_SectionAlignments::OnRun(const UMovieSceneSequence* InSequence, FSequenceValidationResults& OutResults) const
{
	UMovieScene* MovieScene = InSequence->GetMovieScene();

	FSectionAlignmentsImportantTimeVisitor Visitor;

	// Add the root sequence's playback start and end as important times.
	Visitor.AddImportantTimes(
			MovieScene->GetPlaybackRange(), UE::MovieScene::FSubSequenceSpace(),
			LOCTEXT("PlaybackRangeLowerBound", "Playback Range's Lower Bound"),
			LOCTEXT("PlaybackRangeUpperBound", "Playback Range's Upper Bound"));

	// Find important times in the whole hierarchy.
	UE::MovieScene::FSequenceVisitParams VisitParams;
	VisitParams.bVisitRootTracks = true;
	VisitParams.bVisitSubSequences = true;
	UE::MovieScene::VisitSequence(const_cast<UMovieSceneSequence*>(InSequence), VisitParams, Visitor);

	TArray<FImportantFrameTime> ImportantTimes;
	Visitor.GetImportantTimes(ImportantTimes);

	// Re-visit the sequence hierarchy and report on things that are off.
	VisitParams.bVisitObjectBindings = true;
	VisitParams.bVisitSections = true;
	VisitParams.bVisitTracks = true;

	FSectionAlignmentsValidationVisitor Validator(MakeRuleInfo(), ImportantTimes, OutResults);
	Validator.Initialize(InSequence);
	UE::MovieScene::VisitSequence(const_cast<UMovieSceneSequence*>(InSequence), VisitParams, Validator);
}

}  // namespace UE::Sequencer

#undef LOCTEXT_NAMESPACE

