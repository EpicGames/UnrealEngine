// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Validation/SequenceValidationRule.h"

namespace UE::Sequencer
{

/**
 * A validation rule that checks that sections throughout the sequence hierarchy are correctly aligned
 * to "important times" such as camera cuts and start/end times.
 */
class FSequenceValidationRule_SectionAlignments : public FSequenceValidationRule
{
public:

	static FSequenceValidationRuleInfo MakeRuleInfo();

protected:

	virtual void OnRun(const UMovieSceneSequence* InSequence, FSequenceValidationResults& OutResults) const override;
};

}  // namespace UE::Sequencer

