// Copyright Epic Games, Inc. All Rights Reserved.

#include "Validation/Rules/SequenceValidationRule_UnassignedBindingsAndAssets.h"

#include "Algo/BinarySearch.h"
#include "Internationalization/Internationalization.h"
#include "MovieScene.h"
#include "MovieSceneCommonHelpers.h"
#include "MovieSceneSection.h"
#include "MovieSceneSequence.h"
#include "MovieSceneSequenceVisitor.h"
#include "MovieSceneTrack.h"
#include "Sections/MovieSceneAudioSection.h"
#include "Sections/MovieScene3DConstraintSection.h"
#include "Sections/MovieSceneCameraCutSection.h"
#include "Sections/MovieSceneSkeletalAnimationSection.h"
#include "Sections/MovieSceneSubSection.h"
#include "Validation/SequenceValidationResult.h"

#define LOCTEXT_NAMESPACE "SequenceValidationRule_UnassignedBindingsAndAssets"

namespace UE::Sequencer
{

struct FUnassignedBindingsAndAssetsValidationVisitor : public UE::MovieScene::ISequenceVisitor
{
	using FSubSequenceSpace = UE::MovieScene::FSubSequenceSpace;

public:

	FUnassignedBindingsAndAssetsValidationVisitor(const FSequenceValidationRuleInfo& InRuleInfo, FSequenceValidationResults& OutResults)
		: RuleInfo(InRuleInfo)
		, Results(OutResults)
	{
	}

	void Initialize(const UMovieSceneSequence* InSequence)
	{
		UMovieScene* MovieScene = InSequence->GetMovieScene();

		CurrentSubSectionTrail.Reserve(4);
	}

	virtual void VisitStartSubSequence(UMovieSceneSubSection* SubSection, const FGuid& ObjectBindingID, const FSubSequenceSpace& LocalSpace) override
	{
		CurrentSubSectionTrail.Add(SubSection);
	}

	virtual void VisitEndSubSequence(UMovieSceneSubSection* SubSection, const FGuid& ObjectBindingID, const FSubSequenceSpace& LocalSpace) override
	{
		ensure(CurrentSubSectionTrail.Num() > 0 && CurrentSubSectionTrail.Last() == SubSection);
		CurrentSubSectionTrail.Pop(EAllowShrinking::No);
	}

	virtual void VisitSection(UMovieSceneTrack* Track, UMovieSceneSection* Section, const FGuid& ObjectBindingID, const FSubSequenceSpace& LocalSpace) override
	{
		if (UMovieSceneAudioSection* AudioSection = Cast<UMovieSceneAudioSection>(Section))
		{
			if (!AudioSection->GetSound())
			{
				WarnAbout(Results, Section);
			}
		}
		else if (UMovieSceneCameraCutSection* CameraCutSection = Cast<UMovieSceneCameraCutSection>(Section))
		{
			if (!CameraCutSection->GetCameraBindingID().IsValid())
			{
				WarnAbout(Results, Section);
			}
		}
		else if (UMovieScene3DConstraintSection* ConstraintSection = Cast<UMovieScene3DConstraintSection>(Section))
		{
			if (!ConstraintSection->GetConstraintBindingID().IsValid())
			{
				WarnAbout(Results, Section);
			}
		}
		else if (UMovieSceneSubSection* SubSection = Cast<UMovieSceneSubSection>(Section))
		{
			if (!SubSection->GetSequence())
			{
				WarnAbout(Results, Section);
			}
		}
		else if (UMovieSceneSkeletalAnimationSection* SkeletalAnimationSection = Cast<UMovieSceneSkeletalAnimationSection>(Section))
		{
			if (!SkeletalAnimationSection->GetAnimation())
			{
				WarnAbout(Results, Section);
			}
		}
	}

private:

	void WarnAbout(FSequenceValidationResults& OutResults, UMovieSceneSection* Section)
	{
		TSharedRef<FSequenceValidationResult> NewResult = MakeShared<FSequenceValidationResult>(EMessageSeverity::Warning, Section, RuleInfo);
		NewResult->SetSubSectionTrail(CurrentSubSectionTrail);
		NewResult->SetUserMessage(FText::Format(
			LOCTEXT("UnassignedBindingsAndAssetsWarningFmt", "Section {0} : contains an unassigned binding/asset."),
			MovieSceneHelpers::GetDisplayPathName(Section)));
		OutResults.AddResult(NewResult);
	}

private:

	FSequenceValidationRuleInfo RuleInfo;
	TArray<UMovieSceneSubSection*> CurrentSubSectionTrail;
	FSequenceValidationResults& Results;
};

FSequenceValidationRuleInfo FSequenceValidationRule_UnassignedBindingsAndAssets::MakeRuleInfo()
{
	FSequenceValidationRuleInfo RuleInfo;
	RuleInfo.RuleName = LOCTEXT("RuleName", "Unassigned Bindings/Assets");
	RuleInfo.RuleDescription = LOCTEXT("RuleDescription", "Check that animations, audio, subsequences, camera cuts, attach/path sections reference valid assets.");
	RuleInfo.RuleFactory = FOnCreateSequenceValidationRule::CreateLambda([]() { return MakeShared<FSequenceValidationRule_UnassignedBindingsAndAssets>(); });
	RuleInfo.RuleColor = FSequenceValidationRule::GetRuleColor(RuleInfo.RuleName);
	return RuleInfo;
}

void FSequenceValidationRule_UnassignedBindingsAndAssets::OnRun(const UMovieSceneSequence* InSequence, FSequenceValidationResults& OutResults) const
{
	UMovieScene* MovieScene = InSequence->GetMovieScene();

	// Visit the sequence hierarchy and report on things that are off.
	UE::MovieScene::FSequenceVisitParams VisitParams;
	VisitParams.bVisitRootTracks = true;
	VisitParams.bVisitObjectBindings = true;
	VisitParams.bVisitSubSequences = true;
	VisitParams.bVisitTracks = true;
	VisitParams.bVisitSections = true;

	FUnassignedBindingsAndAssetsValidationVisitor Validator(MakeRuleInfo(), OutResults);
	Validator.Initialize(InSequence);
	UE::MovieScene::VisitSequence(const_cast<UMovieSceneSequence*>(InSequence), VisitParams, Validator);
}

}  // namespace UE::Sequencer

#undef LOCTEXT_NAMESPACE

