// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Validation/SequenceValidationRule.h"

namespace UE::Sequencer
{

/**
 * A validation rule that checks that sections reference valid assets.
 */
class FSequenceValidationRule_UnassignedBindingsAndAssets : public FSequenceValidationRule
{
public:

	static FSequenceValidationRuleInfo MakeRuleInfo();

protected:

	virtual void OnRun(const UMovieSceneSequence* InSequence, FSequenceValidationResults& OutResults) const override;
};

}  // namespace UE::Sequencer

