// Copyright Epic Games, Inc. All Rights Reserved.

#include "Validation/Rules/SequenceValidationRule_WholeSectionRanges.h"

#include "Internationalization/Internationalization.h"
#include "MovieScene.h"
#include "MovieSceneCommonHelpers.h"
#include "MovieSceneSection.h"
#include "MovieSceneSequence.h"
#include "MovieSceneSequenceVisitor.h"
#include "Sections/MovieSceneSkeletalAnimationSection.h"
#include "Validation/SequenceValidationResult.h"

#define LOCTEXT_NAMESPACE "SequenceValidationRule_WholeSectionRanges"

namespace UE::Sequencer
{

class FWholeSectionRangeVisitor : public UE::MovieScene::ISequenceVisitor
{
	using FSubSequenceSpace = UE::MovieScene::FSubSequenceSpace;

public:

	FWholeSectionRangeVisitor(const FSequenceValidationRuleInfo& InRuleInfo, FSequenceValidationResults& OutResults)
		: RuleInfo(InRuleInfo)
		, Results(OutResults)
	{
		UpperBoundExcludedSectionClasses.Add(UMovieSceneSkeletalAnimationSection::StaticClass());
	}

	virtual void VisitStartSubSequence(UMovieSceneSubSection* SubSection, const FGuid& ObjectBindingID, const FSubSequenceSpace& LocalSpace) override
	{
		CurrentSubSectionTrail.Add(SubSection);
	}

	virtual void VisitEndSubSequence(UMovieSceneSubSection* SubSection, const FGuid& ObjectBindingID, const FSubSequenceSpace& LocalSpace) override
	{
		ensure(CurrentSubSectionTrail.Num() > 0 && CurrentSubSectionTrail.Last() == SubSection);
		CurrentSubSectionTrail.Pop(EAllowShrinking::No);
	}

	virtual void VisitSection(UMovieSceneTrack* Track, UMovieSceneSection* Section, const FGuid& ObjectBindingID, const FSubSequenceSpace& LocalSpace) override
	{
		UMovieScene* MovieScene = Track->GetTypedOuter<UMovieScene>();
		const FFrameRate& DisplayRate = MovieScene->GetDisplayRate();
		const FFrameRate& TickResolution = MovieScene->GetTickResolution();

		TRange<FFrameNumber> SectionRange = Section->GetRange();
		if (SectionRange.HasLowerBound())
		{
			const FFrameNumber LowerBoundValue = SectionRange.GetLowerBoundValue();
			const FFrameTime StartTime = ConvertFrameTime(LowerBoundValue, TickResolution, DisplayRate);
			if (StartTime.GetSubFrame() != 0.f)
			{
				const bool bIsLowerBound = true;
				AddWarning(Section, LowerBoundValue, bIsLowerBound);
			}
		}
		if (SectionRange.HasUpperBound() && ShouldSectionHaveWholeFrameUpperBound(Section))
		{
			const FFrameNumber UpperBoundValue = SectionRange.GetUpperBoundValue();
			const FFrameTime EndTime = ConvertFrameTime(UpperBoundValue, TickResolution, DisplayRate);
			if (EndTime.GetSubFrame() != 0.f)
			{
				const bool bIsLowerBound = false;
				AddWarning(Section, UpperBoundValue, bIsLowerBound);
			}
		}
	}

private:

	bool ShouldSectionHaveWholeFrameUpperBound(UMovieSceneSection* Section)
	{
		UClass* SectionClass = Section->GetClass();
		for (TSubclassOf<UMovieSceneSection> Class : UpperBoundExcludedSectionClasses)
		{
			if (SectionClass->IsChildOf(Class))
			{
				return false;
			}
		}
		return true;
	}

	void AddWarning(UMovieSceneSection* Section, FFrameTime LocalTime, const bool bIsLowerBound)
	{
		TSharedRef<FSequenceValidationResult> NewResult = MakeShared<FSequenceValidationResult>(EMessageSeverity::Warning, Section, RuleInfo);
		NewResult->SetSubSectionTrail(CurrentSubSectionTrail);
		NewResult->SetLocalTime(LocalTime);
		if (bIsLowerBound)
		{
			NewResult->SetUserMessage(FText::Format(LOCTEXT("WholeFrameLowerBoundWarningFmt", "Section {0} : lower bound is not on a whole frame."), MovieSceneHelpers::GetDisplayPathName(Section)));
		}
		else
		{
			NewResult->SetUserMessage(FText::Format(LOCTEXT("WholeFrameUpperBoundWarningFmt", "Section {0} : upper bound is not on a whole frame."), MovieSceneHelpers::GetDisplayPathName(Section)));
		}
		Results.AddResult(NewResult);
	}

private:

	TArray<TSubclassOf<UMovieSceneSection>> UpperBoundExcludedSectionClasses;
	
	FSequenceValidationRuleInfo RuleInfo;
	TArray<UMovieSceneSubSection*> CurrentSubSectionTrail;
	FSequenceValidationResults& Results;
};

FSequenceValidationRuleInfo FSequenceValidationRule_WholeSectionRanges::MakeRuleInfo()
{
	FSequenceValidationRuleInfo RuleInfo;
	RuleInfo.RuleName = LOCTEXT("RuleName", "Whole Section Ranges");
	RuleInfo.RuleDescription = LOCTEXT("RuleDescription", "Check that most sections start and end on whole frames.");
	RuleInfo.RuleFactory = FOnCreateSequenceValidationRule::CreateLambda([]() { return MakeShared<FSequenceValidationRule_WholeSectionRanges>(); });
	RuleInfo.RuleColor = FSequenceValidationRule::GetRuleColor(RuleInfo.RuleName);
	return RuleInfo;
}

void FSequenceValidationRule_WholeSectionRanges::OnRun(const UMovieSceneSequence* InSequence, FSequenceValidationResults& OutResults) const
{
	UE::MovieScene::FSequenceVisitParams VisitParams;
	VisitParams.bVisitRootTracks = true;
	VisitParams.bVisitObjectBindings = true;
	VisitParams.bVisitSubSequences = true;
	VisitParams.bVisitTracks = true;
	VisitParams.bVisitSections = true;

	FWholeSectionRangeVisitor Visitor(MakeRuleInfo(), OutResults);
	UE::MovieScene::VisitSequence(const_cast<UMovieSceneSequence*>(InSequence), VisitParams, Visitor);
}

}  // namespace UE::Sequencer

#undef LOCTEXT_NAMESPACE

