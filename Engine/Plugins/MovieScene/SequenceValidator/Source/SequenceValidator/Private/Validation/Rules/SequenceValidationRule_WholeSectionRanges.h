// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Validation/SequenceValidationRule.h"

namespace UE::Sequencer
{

/**
 * A validation rule that checks that most sections have start/end times that are on whole frames.
 */
class FSequenceValidationRule_WholeSectionRanges : public FSequenceValidationRule
{
public:

	static FSequenceValidationRuleInfo MakeRuleInfo();

protected:

	virtual void OnRun(const UMovieSceneSequence* InSequence, FSequenceValidationResults& OutResults) const override;
};

}  // namespace UE::Sequencer

