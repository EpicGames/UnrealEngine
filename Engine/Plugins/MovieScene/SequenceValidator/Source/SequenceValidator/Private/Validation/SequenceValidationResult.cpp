// Copyright Epic Games, Inc. All Rights Reserved.

#include "Validation/SequenceValidationResult.h"

#include "Sections/MovieSceneSubSection.h"
#include "Templates/SharedPointer.h"

namespace UE::Sequencer
{

bool FSequenceValidationResult::GetSubSectionTrail(TArray<UMovieSceneSubSection*>& OutTrail) const
{
	bool bSuccess = true;
	for (TWeakObjectPtr<UMovieSceneSubSection> WeakSubSection : WeakSubSectionTrail)
	{
		if (UMovieSceneSubSection* SubSection = WeakSubSection.Get())
		{
			OutTrail.Add(SubSection);
		}
		else
		{
			bSuccess = false;
			break;
		}
	}
	if (!bSuccess)
	{
		OutTrail.Reset();
		return false;
	}
	return true;
}

void FSequenceValidationResult::SetSubSectionTrail(TArrayView<UMovieSceneSubSection*> InTrail)
{
	WeakSubSectionTrail.Reset();
	for (UMovieSceneSubSection* SubSection : InTrail)
	{
		WeakSubSectionTrail.Add(SubSection);
	}
}

TSharedPtr<FSequenceValidationResult> FSequenceValidationResult::GetRoot()
{
	TSharedPtr<FSequenceValidationResult> Current = SharedThis(this);
	while (TSharedPtr<FSequenceValidationResult> Parent = Current->GetParent())
	{
		Current = Parent;
	}
	return Current;
}

void FSequenceValidationResult::AddChild(TSharedRef<FSequenceValidationResult> InChild)
{
	if (ensure(!InChild->WeakParent.IsValid()))
	{
		InChild->WeakParent = SharedThis(this);
		Children.Add(InChild);
	}
}

void FSequenceValidationResult::AppendChildren(TConstArrayView<TSharedPtr<FSequenceValidationResult>> InChildren)
{
	for (TSharedPtr<FSequenceValidationResult> Child : InChildren)
	{
		if (Child && ensure(!Child->WeakParent.IsValid()))
		{
			Child->WeakParent = SharedThis(this);
			Children.Add(Child);
		}
	}
}

void FSequenceValidationResults::AddResult(TSharedRef<FSequenceValidationResult> InResult)
{
	ValidationResults.Add(InResult);
}

void FSequenceValidationResults::AppendResults(const FSequenceValidationResults& InResults)
{
	ValidationResults.Append(InResults.ValidationResults);
}

void FSequenceValidationResults::AppendResults(TConstArrayView<TSharedPtr<FSequenceValidationResult>> InResults)
{
	ValidationResults.Append(InResults);
}

TArrayView<const TSharedPtr<FSequenceValidationResult>> FSequenceValidationResults::GetResults() const
{
	return ValidationResults;
}

void FSequenceValidationResults::Reset()
{
	ValidationResults.Reset();
}

}  // namespace UE::Sequencer

