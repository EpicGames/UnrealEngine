// Copyright Epic Games, Inc. All Rights Reserved.

#include "Validation/SequenceValidationRule.h"

namespace UE::Sequencer
{

FSlateColor FSequenceValidationRule::GetRuleColor(const FText& InRuleName)
{
	if (InRuleName.IsEmpty())
	{
		return FSlateColor::UseForeground();
	}

	static TMap<FString, FSlateColor> RuleColorMap;
	if (!RuleColorMap.Contains(InRuleName.ToString()))
	{
		static int32 Count = 0;
		static TArray<FSlateColor> AvailableRuleColors =
		{
			FSlateColor(FLinearColor::FromSRGBColor(FColor(253, 94, 94, 255))),
			FSlateColor(FLinearColor::FromSRGBColor(FColor(225, 102, 182, 255))),
			FSlateColor(FLinearColor::FromSRGBColor(FColor(187, 107, 240, 255))),
			FSlateColor(FLinearColor::FromSRGBColor(FColor(134, 138, 253, 255))),
			FSlateColor(FLinearColor::FromSRGBColor(FColor(51, 191, 255, 255))),
			FSlateColor(FLinearColor::FromSRGBColor(FColor(68, 213, 191, 255))),
			FSlateColor(FLinearColor::FromSRGBColor(FColor(95, 227, 103, 255))),
			FSlateColor(FLinearColor::FromSRGBColor(FColor(170, 231, 85, 255))),
			FSlateColor(FLinearColor::FromSRGBColor(FColor(247, 216, 43, 255))),
			FSlateColor(FLinearColor::FromSRGBColor(FColor(249, 169, 31, 255))),
			FSlateColor(FLinearColor::FromSRGBColor(FColor(254, 133, 57, 255))),
			FSlateColor(FLinearColor::FromSRGBColor(FColor(169, 129, 102, 255)))
		};
			
		RuleColorMap.Add(InRuleName.ToString(), AvailableRuleColors[Count]);

		Count++;
		if (Count >= AvailableRuleColors.Num())
		{
			Count = 0;
		}
	}
	return RuleColorMap[InRuleName.ToString()];
}

void FSequenceValidationRule::Run(const UMovieSceneSequence* InSequence, FSequenceValidationResults& OutResults) const
{
	OnRun(InSequence, OutResults);
}

}  // namespace UE::Sequencer

