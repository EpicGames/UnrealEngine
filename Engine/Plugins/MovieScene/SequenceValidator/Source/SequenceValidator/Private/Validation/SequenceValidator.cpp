// Copyright Epic Games, Inc. All Rights Reserved.

#include "Validation/SequenceValidator.h"

#include "Algo/AllOf.h"
#include "ISequenceValidatorModule.h"
#include "Modules/ModuleManager.h"
#include "MovieSceneFwd.h"
#include "Tasks/Task.h"
#include "Tasks/TaskPrivate.h"
#include "Validation/SequenceValidationResult.h"
#include "Validation/SequenceValidationRule.h"

#define LOCTEXT_NAMESPACE "SequenceValidator"

namespace UE::Sequencer
{

static TAutoConsoleVariable<int32> CVarSequenceValidationMaxConcurrentTasks(
	TEXT("SequenceValidation.MaxConcurrentTasks"),
	-1,
	TEXT("The maximum number of async tasks running sequence validation rules at the same time.")
	TEXT("Default: -1 (which means to use the number of cores available."),
	ECVF_Default);

namespace Internal
{

class FSequenceValidatorTaskScheduler;

/**
 * Class responsible for running a single validation rule on a single sequence.
 */
struct FSequenceValidatorTask
{
	const UMovieSceneSequence* Sequence = nullptr;
	TSharedPtr<const FSequenceValidationRule> ValidationRule;

	FSequenceValidationResults TaskResults;

	FSequenceValidatorTask(const UMovieSceneSequence* InSequence, TSharedPtr<const FSequenceValidationRule> InRule)
		: Sequence(InSequence)
		, ValidationRule(InRule)
	{
	}

	void Run();
};

/**
 * A class that is able to run validation tasks dispatched by the FSequenceValidatorTaskScheduler.
 */
class FSequenceValidatorTaskRunner
{
public:

	FSequenceValidatorTaskRunner(int32 InRunnerIndex) : RunnerIndex(InRunnerIndex) {}

	void Start(FSequenceValidatorTaskScheduler& InScheduler);

	bool IsRunning() const { return bIsRunning; }

private:

	void ThreadedRun(FSequenceValidatorTaskScheduler& InScheduler);

private:

	int32 RunnerIndex = INDEX_NONE;
	bool bIsRunning = false;
};

/**
 * Main class for running a set of validation rules on a set of sequences in a parallel fashion.
 */
class FSequenceValidatorTaskScheduler
{
public:

	FSequenceValidatorTaskScheduler(TArray<TSharedPtr<FSequenceValidationRuleInfo>> InRuleInfos)
		: ValidationRuleInfos(InRuleInfos)
	{
	}

	~FSequenceValidatorTaskScheduler()
	{
		ensure(!MainSignal);
	}

	void RunQueue(TArrayView<UMovieSceneSequence*> InQueue, FSimpleDelegate&& InOnFinished);

	const FSequenceValidationResults& GetResults() const { return Results; }

private:

	FSequenceValidatorTask* AcquireNextTask(int32 InRunnerIndex);
	void OnRunnerFinished(int32 InRunnerIndex);
	void GatherResults(TArrayView<UMovieSceneSequence*> InQueue);

private:

	TArray<TSharedPtr<FSequenceValidationRuleInfo>> ValidationRuleInfos;
	FSequenceValidationResults Results;

	// Transient fields for running a queue.

	TArray<FSequenceValidatorTask> Tasks;

	int32 NextTaskIndex = 0;
	int32 NumRemainingRunners = 0;
	FEvent* MainSignal = nullptr;

	friend class FSequenceValidatorTaskRunner;
};

void FSequenceValidatorTask::Run()
{
	if (Sequence && ValidationRule)
	{
		ValidationRule->Run(Sequence, TaskResults);
	}
}

void FSequenceValidatorTaskRunner::Start(FSequenceValidatorTaskScheduler& InScheduler)
{
	UE_LOGF(LogSequenceValidator, Log, "Starting sequence validator runner %d", RunnerIndex);
	bIsRunning = true;

	UE::Tasks::Launch(
			TEXT("SequenceValidatorTaskRunner"),
			[this, &InScheduler]() { this->ThreadedRun(InScheduler); },
			UE::Tasks::ETaskPriority::Default);
}

void FSequenceValidatorTaskRunner::ThreadedRun(FSequenceValidatorTaskScheduler& InScheduler)
{
	for (;;)
	{
		FSequenceValidatorTask* NextTask = InScheduler.AcquireNextTask(RunnerIndex);
		if (NextTask)
		{
			NextTask->Run();
		}
		else
		{
			break;
		}
	}

	UE_LOGF(LogSequenceValidator, Log, "Stopping sequence validator runner %d", RunnerIndex);
	bIsRunning = false;
	InScheduler.OnRunnerFinished(RunnerIndex);
}

void FSequenceValidatorTaskScheduler::RunQueue(TArrayView<UMovieSceneSequence*> InQueue, FSimpleDelegate&& InOnFinished)
{
	ensure(!MainSignal);
	MainSignal = FPlatformProcess::GetSynchEventFromPool();

	// Create a task for each sequence/rule pair, for rules that are enabled.
	Tasks.Reset();
	for (const UMovieSceneSequence* Sequence : InQueue)
	{
		for (TSharedPtr<FSequenceValidationRuleInfo> RuleInfo : ValidationRuleInfos)
		{
			if (RuleInfo && RuleInfo->bIsEnabled)
			{
				TSharedRef<FSequenceValidationRule> Rule = RuleInfo->RuleFactory.Execute();
				Tasks.Emplace(Sequence, Rule);
			}
		}
	}

	const int32 ConfiguredMaxConcurrentTasks = CVarSequenceValidationMaxConcurrentTasks.GetValueOnAnyThread();
	const int32 NumConcurrentTasks = ConfiguredMaxConcurrentTasks == -1 ? FPlatformMisc::NumberOfCores() : ConfiguredMaxConcurrentTasks;

	NextTaskIndex = 0;
	NumRemainingRunners = NumConcurrentTasks;

	UE_LOGF(LogSequenceValidator, Log, "Preparing %d sequence validator runners", NumConcurrentTasks);
	TArray<FSequenceValidatorTaskRunner> Runners;
	Runners.Reserve(NumConcurrentTasks); // Important: we don't want runners to be re-allocated elsewhere once they start.
	for (int32 RunnerIndex = 0; RunnerIndex < NumConcurrentTasks; ++RunnerIndex)
	{
		FSequenceValidatorTaskRunner& Runner = Runners.Emplace_GetRef(RunnerIndex);
		Runner.Start(*this);
	}

	if (ensure(MainSignal))
	{
		MainSignal->Wait();
	}

	UE_LOGF(LogSequenceValidator, Log, "All sequence validator runners finished!");
	const bool bAllRunnersFinished = Algo::AllOf(Runners, 
			[](const FSequenceValidatorTaskRunner& InRunner)
			{
				return !InRunner.IsRunning();
			});
	ensure(bAllRunnersFinished);

	if (ensure(MainSignal))
	{
		FPlatformProcess::ReturnSynchEventToPool(MainSignal);
		MainSignal = nullptr;
	}

	GatherResults(InQueue);

	Tasks.Reset();

	InOnFinished.ExecuteIfBound();
}

void FSequenceValidatorTaskScheduler::GatherResults(TArrayView<UMovieSceneSequence*> InQueue)
{
	Results.Reset();

	// Create one root validation result for each root sequence. Add them to our results as we go.
	TMap<UMovieSceneSequence*, TSharedPtr<FSequenceValidationResult>> SequenceToResult;
	for (UMovieSceneSequence* Sequence : InQueue)
	{
		TSharedRef<FSequenceValidationResult> SequenceResult = MakeShared<FSequenceValidationResult>((UObject*)Sequence);
		SequenceToResult.Add(Sequence, SequenceResult);
		Results.AddResult(SequenceResult);
	}

	// Match validation results from the tasks to their owning sequence result.
	for (FSequenceValidatorTask& Task : Tasks)
	{
		if (TSharedPtr<FSequenceValidationResult> SequenceResult = SequenceToResult.FindRef(Task.Sequence))
		{
			SequenceResult->AppendChildren(Task.TaskResults.GetResults());
		}
	}

	// Add "empty" result for sequences with no messages.
	for (UMovieSceneSequence* Sequence : InQueue)
	{
		TSharedPtr<FSequenceValidationResult> SequenceResult = SequenceToResult.FindRef(Sequence);
		if (SequenceResult && !SequenceResult->HasChildren())
		{
			TSharedRef<FSequenceValidationResult> EmptyResult = MakeShared<FSequenceValidationResult>(
					EMessageSeverity::Info, LOCTEXT("EmptyResultMessage", "No issues found."));
			SequenceResult->AddChild(EmptyResult);
		}
	}
}

FSequenceValidatorTask* FSequenceValidatorTaskScheduler::AcquireNextTask(int32 InRunnerIndex)
{
	const int32 NewNextTaskIndex = FPlatformAtomics::InterlockedIncrement(&NextTaskIndex);
	const int32 CurrentTaskIndex = NewNextTaskIndex - 1;
	if (CurrentTaskIndex < Tasks.Num())
	{
		UE_LOGF(LogSequenceValidator, Log, "Dispatching sequence validation task %d to runner %d", CurrentTaskIndex, InRunnerIndex);
		return &Tasks[CurrentTaskIndex];
	}
	return nullptr;
}

void FSequenceValidatorTaskScheduler::OnRunnerFinished(int32 InRunnerIndex)
{
	const int32 NewNumRemainingRunners = FPlatformAtomics::InterlockedDecrement(&NumRemainingRunners);
	UE_LOGF(LogSequenceValidator, Log, "Sequence validator runner %d finished, %d runners remaining", InRunnerIndex, NewNumRemainingRunners);
	if (NewNumRemainingRunners == 0)
	{
		MainSignal->Trigger();
	}
}

}  // namespace Internal

FSequenceValidator::FSequenceValidator()
{
	CreateValidationRuleInfos();
}

FSequenceValidator::~FSequenceValidator()
{
}

TArrayView<UMovieSceneSequence* const> FSequenceValidator::GetQueue() const
{
	return ValidationQueue;
}

TArray<TSharedPtr<FSequenceValidationRuleInfo>> FSequenceValidator::GetRules() const
{
	TArray<TSharedPtr<FSequenceValidationRuleInfo>> Rules;
	for (TSharedRef<FSequenceValidationRuleInfo> ValidationRuleInfo : ValidationRuleInfos)
	{
		Rules.Add(ValidationRuleInfo.ToSharedPtr());
	}
	return Rules;
}

const FSequenceValidationResults& FSequenceValidator::GetResults() const
{
	return ValidationResults;
}

void FSequenceValidator::Queue(UMovieSceneSequence* InSequence)
{
	ValidationQueue.AddUnique(InSequence);
}

void FSequenceValidator::Queue(TArrayView<UMovieSceneSequence*> InSequences)
{
	for (UMovieSceneSequence* Sequence : InSequences)
	{
		ValidationQueue.AddUnique(Sequence);
	}
}

void FSequenceValidator::Delete(UMovieSceneSequence* InSequence)
{
	ValidationQueue.Remove(InSequence);
}

void FSequenceValidator::ClearQueue()
{
	ValidationQueue.Reset();
}

void FSequenceValidator::Validate(UMovieSceneSequence* InSequence)
{
	Queue(InSequence);
	StartValidation(false);
	ClearQueue();
}

void FSequenceValidator::Validate(TArrayView<UMovieSceneSequence*> InSequences)
{
	Queue(InSequences);
	StartValidation(false);
	ClearQueue();
}

void FSequenceValidator::StartValidation()
{
	StartValidation(true);
}

bool FSequenceValidator::IsValidating() const
{
	return Scheduler.IsValid();
}

FSimpleMulticastDelegate& FSequenceValidator::GetOnValidationFinished()
{
	return OnValidationFinishedDelegate;
}

void FSequenceValidator::CreateValidationRuleInfos()
{
	ISequenceValidatorModule& ValidationModule = FModuleManager::LoadModuleChecked<ISequenceValidatorModule>("SequenceValidator");
	
	for (const FSequenceValidationRuleInfo& ValidationRuleInfo : ValidationModule.GetValidationRules())
	{
		ValidationRuleInfos.Add(MakeShared<FSequenceValidationRuleInfo>(ValidationRuleInfo));
	}
}

void FSequenceValidator::StartValidation(bool bAsync)
{
	if (Scheduler.IsValid())
	{
		return;
	}

	ValidationResults.Reset();

	if (ValidationQueue.IsEmpty())
	{
		return;
	}

	using namespace Internal;

	Scheduler = MakeUnique<FSequenceValidatorTaskScheduler>(GetRules());
	
	if (bAsync)
	{
		UE::Tasks::Launch(TEXT("SequenceValidatorMain"),
				[this]() 
				{ 
					Scheduler->RunQueue(
							ValidationQueue,
							FSimpleDelegate::CreateRaw(this, &FSequenceValidator::OnValidationFinished));
				},
				UE::Tasks::ETaskPriority::Normal);
	}
	else
	{
		Scheduler->RunQueue(
				ValidationQueue,
				FSimpleDelegate::CreateRaw(this, &FSequenceValidator::OnValidationFinished));
	}
}

void FSequenceValidator::OnValidationFinished()
{
	ValidationResults = Scheduler->GetResults();
	Scheduler.Reset();
}

}  // namespace UE::Sequencer

#undef LOCTEXT_NAMESPACE

