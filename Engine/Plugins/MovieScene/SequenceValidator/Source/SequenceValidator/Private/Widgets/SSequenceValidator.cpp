// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SSequenceValidator.h"

#include "Framework/Application/SlateApplication.h"
#include "Framework/Docking/TabManager.h"
#include "Framework/Views/ITypedTableView.h"
#include "MovieSceneSequence.h"
#include "SequenceValidatorCommands.h"
#include "SequenceValidatorStyle.h"
#include "SPrimaryButton.h"
#include "ToolMenus.h"
#include "Validation/SequenceValidator.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Layout/SSplitter.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/SSequenceValidatorQueue.h"
#include "Widgets/SSequenceValidatorResults.h"
#include "Widgets/SSequenceValidatorRules.h"
#include "WorkspaceMenuStructure.h"
#include "WorkspaceMenuStructureModule.h"

#define LOCTEXT_NAMESPACE "SSequenceValidator"

namespace UE::Sequencer
{

const FName SSequenceValidator::WindowName(TEXT("SequenceValidator"));
const FName SSequenceValidator::MenubarName(TEXT("SequenceValidator.Menubar"));

void SSequenceValidator::RegisterTabSpawners()
{
	TSharedRef<FSequenceValidatorStyle> ValidatorStyle = FSequenceValidatorStyle::Get();

	FGlobalTabmanager::Get()->RegisterNomadTabSpawner(
		SSequenceValidator::WindowName,
		FOnSpawnTab::CreateStatic(&SSequenceValidator::SpawnSequenceValidator)
	)
	.SetDisplayName(LOCTEXT("TabDisplayName", "Sequence Validator"))
	.SetTooltipText(LOCTEXT("TabTooltipText", "Open the Sequence Validator tab."))
	.SetIcon(FSlateIcon(ValidatorStyle->GetStyleSetName(), "SequenceValidator.TabIcon"))
	.SetGroup(WorkspaceMenu::GetMenuStructure().GetLevelEditorCinematicsCategory())
	.SetCanSidebarTab(false);
}

TSharedRef<SDockTab> SSequenceValidator::SpawnSequenceValidator(const FSpawnTabArgs& Args)
{
	auto NomadTab = SNew(SDockTab)
		.TabRole(ETabRole::NomadTab)
		.Label(LOCTEXT("TabTitle", "Sequence Validator"));

	TSharedRef<SWidget> MainWidget = SNew(SSequenceValidator);
	NomadTab->SetContent(MainWidget);
	return NomadTab;
}

void SSequenceValidator::UnregisterTabSpawners()
{
	if (FSlateApplication::IsInitialized())
	{
		FGlobalTabmanager::Get()->UnregisterNomadTabSpawner(SSequenceValidator::WindowName);
	}
}

void SSequenceValidator::Construct(const FArguments& InArgs)
{
	Validator = MakeShared<FSequenceValidator>();

	// Setup commands.
	const FSequenceValidatorCommands& Commands = FSequenceValidatorCommands::Get();
	TSharedRef<FUICommandList> CommandList = MakeShareable(new FUICommandList);
	CommandList->MapAction(
			Commands.StartValidation,
			FExecuteAction::CreateSP(this, &SSequenceValidator::StartValidation),
			FCanExecuteAction::CreateSP(this, &SSequenceValidator::CanStartValidation));

	// Build all UI elements.
	TSharedRef<SWidget> MenubarContents = ConstructMenubar();

	// Main layout.
	ChildSlot
	[
		SNew(SVerticalBox)
		+SVerticalBox::Slot()
		.AutoHeight()
		[
			MenubarContents
		]
		+SVerticalBox::Slot()
		.Padding(2.0)
		.FillHeight(1.f)
		[
			SNew(SSplitter)
			.Orientation(Orient_Horizontal)
			+SSplitter::Slot()
			.Value(0.4)
			[
				SNew(SSplitter)
				.Orientation(Orient_Vertical)
				+ SSplitter::Slot()
				[
					SAssignNew(QueueWidget, SSequenceValidatorQueue)
					.Validator(Validator)
				]
				+ SSplitter::Slot()
				[
					SAssignNew(RulesWidget, SSequenceValidatorRules)
					.Validator(Validator)
				]
			]
			+SSplitter::Slot()
			.Value(0.6)
			[
				SAssignNew(ResultsWidget, SSequenceValidatorResults)
				.Validator(Validator)
			]
		]
		+SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Left)
			.Padding(4.f, 8.f)
			[
				SNew(STextBlock)
				.Text(this, &SSequenceValidator::GetValidationStatusText)
			]

			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Right)
			.Padding(4.f, 8.f)
			[
				SNew(SPrimaryButton)
				.Text(LOCTEXT("StartValidation", "Start Validation"))
				.ToolTipText(LOCTEXT("StartValidationTooltip", "Start validating the sequences in the queue"))
				.OnClicked(FOnClicked::CreateLambda([this]() { StartValidation(); return FReply::Handled(); }))
				.IsEnabled_Lambda([this]() -> bool { return this->CanStartValidation(); })
			]
		]
	];
}

TSharedRef<SWidget> SSequenceValidator::ConstructMenubar()
{
	UToolMenus* ToolMenus = UToolMenus::Get();
	if (!ToolMenus->IsMenuRegistered(SSequenceValidator::MenubarName))
	{
		FToolMenuOwnerScoped ToolMenuOwnerScope(this);

		UToolMenu* Menubar = ToolMenus->RegisterMenu(
				SSequenceValidator::MenubarName, NAME_None, EMultiBoxType::MenuBar);
	}

	FToolMenuContext MenubarContext;
	return ToolMenus->GenerateWidget(SSequenceValidator::MenubarName, MenubarContext);
}

void SSequenceValidator::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	SCompoundWidget::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);

	if (bWaitingForValidationToFinish && !Validator->IsValidating())
	{
		bWaitingForValidationToFinish = false;

		ResultsWidget->RequestListRefresh();
	}
}

void SSequenceValidator::StartValidation()
{
	bWaitingForValidationToFinish = true;
	Validator->StartValidation();
}

bool SSequenceValidator::CanStartValidation() const
{
	return !Validator->IsValidating();
}

EVisibility SSequenceValidator::GetValidationStatusVisibility() const
{
	return Validator->IsValidating() ? EVisibility::Visible : EVisibility::Collapsed;
}

FText SSequenceValidator::GetValidationStatusText() const
{
	return Validator->IsValidating() ? 
		FText::Format(LOCTEXT("ValidationStatus_Running", "Validating {0} sequences..."), Validator->GetQueue().Num()) :
		LOCTEXT("ValidationStatus_Waiting", "Ready to validate");
}

}  // namespace UE::Sequencer

#undef LOCTEXT_NAMESPACE

