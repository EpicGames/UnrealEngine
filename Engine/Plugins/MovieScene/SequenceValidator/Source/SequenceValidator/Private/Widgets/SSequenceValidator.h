// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ContentBrowserDelegates.h"
#include "Widgets/SCompoundWidget.h"

#include "SSequenceValidator.generated.h"

class FSpawnTabArgs;
class FUICommandList;
class SDockTab;
struct FAssetData;
struct FToolMenuContext;

namespace UE::Sequencer
{

class FSequenceValidator;
class SSequenceValidatorQueue;
class SSequenceValidatorResults;
class SSequenceValidatorRules;

class SSequenceValidator : public SCompoundWidget
{
public:

	static const FName WindowName;
	static const FName MenubarName;

	static void RegisterTabSpawners();
	static TSharedRef<SDockTab> SpawnSequenceValidator(const FSpawnTabArgs& Args);
	static void UnregisterTabSpawners();

public:

	SLATE_BEGIN_ARGS(SSequenceValidator)
	{}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

protected:

	// SWidget interface.
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

private:

	TSharedRef<SWidget> ConstructMenubar();

	void StartValidation();
	bool CanStartValidation() const;

	EVisibility GetValidationStatusVisibility() const;
	FText GetValidationStatusText() const;

private:

	TSharedPtr<SSequenceValidatorQueue> QueueWidget;
	TSharedPtr<SSequenceValidatorResults> ResultsWidget;
	TSharedPtr<SSequenceValidatorRules> RulesWidget;

	TSharedPtr<FSequenceValidator> Validator;

	bool bWaitingForValidationToFinish = false;
};

}  // namespace UE::Sequencer

UCLASS()
class USequenceValidatorMenuContext : public UObject
{
	GENERATED_BODY()

public:

	TWeakPtr<UE::Sequencer::SSequenceValidator> ValidatorWidget;
};

