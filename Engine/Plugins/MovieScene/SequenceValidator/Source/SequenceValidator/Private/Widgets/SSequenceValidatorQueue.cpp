// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SSequenceValidatorQueue.h"

#include "AssetThumbnail.h"
#include "DragAndDrop/AssetDragDropOp.h"
#include "IContentBrowserSingleton.h"
#include "SequenceValidatorCommands.h"
#include "SequenceValidatorStyle.h"
#include "SPositiveActionButton.h"
#include "ThumbnailRendering/ThumbnailManager.h"
#include "Validation/SequenceValidator.h"
#include "Widgets/SBoxPanel.h"

#define LOCTEXT_NAMESPACE "SSequenceValidatorQueue"

namespace UE::Sequencer
{

class SSequenceValidatorQueueEntry : public STableRow<UMovieSceneSequence*>
{
public:

	SLATE_BEGIN_ARGS(SSequenceValidatorQueueEntry)
	{}
		SLATE_ARGUMENT(TWeakObjectPtr<UMovieSceneSequence>, Item)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& OwnerTable);

private:

	const FSlateBrush* GetThumbnailBorder() const;
	FText GetSequenceName() const;

private:

	TWeakObjectPtr<UMovieSceneSequence> WeakItem;

	TSharedPtr<FAssetThumbnail> AssetThumbnail;
	TSharedPtr<SBorder> ThumbnailBorder;
};

void SSequenceValidatorQueueEntry::Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& OwnerTable)
{
	using FSuperRowType = STableRow<UMovieSceneSequence*>;

	WeakItem = InArgs._Item;

	FIntPoint ThumbnailSize(48, 48);
	TSharedPtr<FAssetThumbnailPool> ThumbnailPool = UThumbnailManager::Get().GetSharedThumbnailPool();

	FAssetData AssetData(WeakItem.Get());
	AssetThumbnail = MakeShared<FAssetThumbnail>(AssetData, ThumbnailSize.X, ThumbnailSize.Y, ThumbnailPool);
	FAssetThumbnailConfig AssetThumbnailConfig;

	ChildSlot
	.Padding(0, 3, 5, 0)
	.VAlign(VAlign_Center)
	[
		SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew(SBorder)
			.Padding(0, 0, 4, 4)
			.Visibility(EVisibility::SelfHitTestInvisible)
			.BorderImage(FAppStyle::Get().GetBrush("PropertyEditor.AssetTileItem.DropShadow"))
			[
				SNew(SOverlay)
				+SOverlay::Slot()
				.Padding(1.0f)
				[
					SAssignNew(ThumbnailBorder, SBorder)
					.Padding(0)
					.BorderImage(FStyleDefaults::GetNoBrush())
					[
						SNew(SBox)
						.WidthOverride(ThumbnailSize.X)
						.HeightOverride(ThumbnailSize.Y)
						[
							AssetThumbnail->MakeThumbnailWidget(AssetThumbnailConfig)
						]
					]
				]
				+SOverlay::Slot()
				[
					SNew(SImage)
					.Image(this, &SSequenceValidatorQueueEntry::GetThumbnailBorder)
					.Visibility(EVisibility::SelfHitTestInvisible)
				]
			]
		]
		+SHorizontalBox::Slot()
		.Padding(8, 4, 4, 4)
		.FillWidth(1.f)
		[
			SNew(STextBlock)
			.Text(this, &SSequenceValidatorQueueEntry::GetSequenceName)
		]
	];

	FSuperRowType::ConstructInternal(
		FSuperRowType::FArguments(),
		OwnerTable);
}

const FSlateBrush* SSequenceValidatorQueueEntry::GetThumbnailBorder() const
{
	static const FName HoveredBorderName("PropertyEditor.AssetThumbnailBorderHovered");
	static const FName RegularBorderName("PropertyEditor.AssetThumbnailBorder");

	return ThumbnailBorder->IsHovered() ? FAppStyle::Get().GetBrush(HoveredBorderName) : FAppStyle::Get().GetBrush(RegularBorderName);
}

FText SSequenceValidatorQueueEntry::GetSequenceName() const
{
	if (UMovieSceneSequence* Item = WeakItem.Get())
	{
		return Item->GetDisplayName();
	}
	return FText::GetEmpty();
}

void SSequenceValidatorQueue::Construct(const FArguments& InArgs)
{
	Validator = InArgs._Validator;

	TSharedRef<FSequenceValidatorStyle> ValidatorStyle = FSequenceValidatorStyle::Get();

	ChildSlot
	[
		SNew(SVerticalBox)
		+SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SBorder)
			.Padding(0.f)
			.BorderImage(FAppStyle::Get().GetBrush("ToolPanel.GroupBorder"))
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.HAlign(HAlign_Left)
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(8.f, 4.f)
					.VAlign(VAlign_Center)
					[
						SNew(SImage)
						.Image(ValidatorStyle->GetBrush("SequenceValidator.QueueTitleIcon"))
					]
					+ SHorizontalBox::Slot()
					.Padding(0.f, 4.f)
					.FillWidth(1.f)
					[
						SNew(STextBlock)
						.Margin(4.f)
						.Text(LOCTEXT("QueueTitle", "Queue"))
						.TextStyle(ValidatorStyle, "SequenceValidator.PanelTitleText")
					]
				]

				+ SHorizontalBox::Slot()
				.HAlign(HAlign_Right)
				.Padding(8.f, 8.f)
				[
					SNew(SPositiveActionButton)
					.Text(LOCTEXT("AddSequence_Label", "Add"))
					.ToolTipText(LOCTEXT("AddSequence_ToolTip", "Add a sequence to the queue"))
					.OnGetMenuContent(this, &SSequenceValidatorQueue::GenerateAddSequenceMenu)
				]
			]
		]
		+SVerticalBox::Slot()
		.FillHeight(1.f)
		[
			SAssignNew(ListView, SListView<UMovieSceneSequence*>)
			.ListItemsSource(&ItemSource)
			.OnGenerateRow(this, &SSequenceValidatorQueue::OnListGenerateItemRow)
			.OnContextMenuOpening(this, &SSequenceValidatorQueue::GetContextMenuContent)
		]
	];
}


TSharedRef<SWidget> SSequenceValidatorQueue::GenerateAddSequenceMenu()
{
	FAssetPickerConfig PickerConfig;
	PickerConfig.bAllowNullSelection = false;
	PickerConfig.bFocusSearchBoxWhenOpened = true;
	PickerConfig.Filter.bRecursiveClasses = true;
	PickerConfig.Filter.ClassPaths.Add(UMovieSceneSequence::StaticClass()->GetClassPathName());
	PickerConfig.SelectionMode = ESelectionMode::Single;
	PickerConfig.InitialAssetViewType = EAssetViewType::List;
	PickerConfig.OnAssetSelected = FOnAssetSelected::CreateSP(this, &SSequenceValidatorQueue::OnAddSequence);
	PickerConfig.OnAssetsActivated = FOnAssetsActivated::CreateSP(this, &SSequenceValidatorQueue::OnAddSequences);

	return IContentBrowserSingleton::Get().CreateAssetPicker(PickerConfig);
}

void SSequenceValidatorQueue::OnAddSequence(const FAssetData& SelectedAsset)
{
	if (UMovieSceneSequence* Sequence = Cast<UMovieSceneSequence>(SelectedAsset.GetAsset()))
	{
		Validator->Queue(Sequence);
	}

	RequestListRefresh();
}

void SSequenceValidatorQueue::OnAddSequences(const TArray<FAssetData>& ActivatedAssets, EAssetTypeActivationMethod::Type ActivationMethod)
{
	for (const FAssetData& AssetData : ActivatedAssets)
	{
		if (UMovieSceneSequence* Sequence = Cast<UMovieSceneSequence>(AssetData.GetAsset()))
		{
			Validator->Queue(Sequence);
		}
	}

	RequestListRefresh();
}

void SSequenceValidatorQueue::DeleteSequences()
{
	for (UMovieSceneSequence* Sequence : ListView->GetSelectedItems())
	{
		Validator->Delete(Sequence);
	}
	RequestListRefresh();
}

void SSequenceValidatorQueue::ClearQueue()
{
	Validator->ClearQueue();

	RequestListRefresh();
}

TSharedRef<ITableRow> SSequenceValidatorQueue::OnListGenerateItemRow(UMovieSceneSequence* Item, const TSharedRef<STableViewBase>& OwnerTable)
{
	return SNew(SSequenceValidatorQueueEntry, OwnerTable)
		.Item(Item);
}

TSharedPtr<SWidget> SSequenceValidatorQueue::GetContextMenuContent()
{
	FMenuBuilder MenuBuilder(true, nullptr);
	MenuBuilder.BeginSection("Edit");
	MenuBuilder.AddMenuEntry(
		LOCTEXT("DeleteFromQueue", "Delete from Queue"),
		LOCTEXT("DeleteFromQueue_Tooltip", "Delete the selected sequences from the queue"),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateSP(this, &SSequenceValidatorQueue::DeleteSequences),
			FCanExecuteAction::CreateLambda([this] { return ListView->GetNumItemsSelected() > 0; })
		),
		NAME_None,
		EUserInterfaceActionType::Button
	);
	MenuBuilder.AddMenuEntry(
		LOCTEXT("ClearQueue", "Clear Queue"),
		LOCTEXT("ClearQueue_Tooltip", "Clear all the sequences from the queue"),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateSP(this, &SSequenceValidatorQueue::ClearQueue)
		),
		NAME_None,
		EUserInterfaceActionType::Button
	);
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

void SSequenceValidatorQueue::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	SCompoundWidget::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);
	
	if (bUpdateItemSource)
	{
		bUpdateItemSource = false;

		UpdateItemSource();

		ListView->RequestListRefresh();
	}
}

void SSequenceValidatorQueue::OnDragEnter(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	SCompoundWidget::OnDragEnter(MyGeometry, DragDropEvent);
}

FReply SSequenceValidatorQueue::OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	TArray<UMovieSceneSequence*> DroppedSequences;

	if (TSharedPtr<FAssetDragDropOp> AssetDragDropOp = DragDropEvent.GetOperationAs<FAssetDragDropOp>())
	{
		const TArray<FAssetData>& Assets = AssetDragDropOp->GetAssets();
		for (const FAssetData& Asset : Assets)
		{
			const UClass* AssetClass = Asset.GetClass();
			if (AssetClass && AssetClass->IsChildOf<UMovieSceneSequence>())
			{
				if (UMovieSceneSequence* Sequence = Cast<UMovieSceneSequence>(Asset.GetAsset()))
				{
					DroppedSequences.Add(Sequence);
				}
			}
		}
	}

	if (!DroppedSequences.IsEmpty())
	{
		Validator->Queue(DroppedSequences);
		RequestListRefresh();

		return FReply::Handled();
	}

	return SCompoundWidget::OnDrop(MyGeometry, DragDropEvent);
}

void SSequenceValidatorQueue::RequestListRefresh()
{
	bUpdateItemSource = true;
}

void SSequenceValidatorQueue::UpdateItemSource()
{
	ItemSource.Reset();

	for (UMovieSceneSequence* Sequence : Validator->GetQueue())
	{
		ItemSource.Add(Sequence);
	}
}

}  // namespace UE::Sequencer

#undef LOCTEXT_NAMESPACE

