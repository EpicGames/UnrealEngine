// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IAssetTypeActions.h"
#include "MovieSceneSequence.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/SListView.h"

namespace UE::Sequencer
{

class FSequenceValidator;

class SSequenceValidatorQueue : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SSequenceValidatorQueue)
	{}
		SLATE_ARGUMENT(TSharedPtr<FSequenceValidator>, Validator)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

	void RequestListRefresh();

protected:

	// SWidget interface.
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
	virtual void OnDragEnter(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;
	virtual FReply OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;

private:

	TSharedRef<SWidget> GenerateAddSequenceMenu();
	void OnAddSequence(const FAssetData& SelectedAsset);
	void OnAddSequences(const TArray<FAssetData>& ActivatedAssets, EAssetTypeActivationMethod::Type ActivationMethod);
	void DeleteSequences();
	void ClearQueue();

	TSharedRef<ITableRow> OnListGenerateItemRow(UMovieSceneSequence* Item, const TSharedRef<STableViewBase>& OwnerTable);
	TSharedPtr<SWidget> GetContextMenuContent();

	void UpdateItemSource();

private:

	TSharedPtr<FSequenceValidator> Validator;

	TSharedPtr<SListView<UMovieSceneSequence*>> ListView;
	TArray<UMovieSceneSequence*> ItemSource;

	bool bUpdateItemSource = false;
};

}  // namespace UE::Sequencer

