// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SSequenceValidatorResults.h"

#include "Editor.h"
#include "ILevelSequenceEditorToolkit.h"
#include "ISequencer.h"
#include "LevelSequence.h"
#include "Misc/ScopedSlowTask.h"
#include "MovieSceneSection.h"
#include "MovieSceneSequence.h"
#include "MovieSceneSequenceID.h"
#include "MovieSceneTrack.h"
#include "SequenceValidatorStyle.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "Validation/SequenceValidationResult.h"
#include "Validation/SequenceValidator.h"

#include "MVVM/Selection/Selection.h"
#include "MVVM/ViewModels/ChannelModel.h"
#include "MVVM/ViewModels/SectionModel.h"
#include "MVVM/ViewModels/SequencerEditorViewModel.h"

#define LOCTEXT_NAMESPACE "SSequenceValidatorResults"

namespace UE::Sequencer
{

class SSequenceValidatorResultEntry : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SSequenceValidatorResultEntry)
	{}
		SLATE_ARGUMENT(TSharedPtr<FSequenceValidationResult>, Item)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

private:

	const FSlateBrush* GetIcon() const;
	FSlateColor GetColorAndOpacity() const;
	FText GetMessage() const;

private:

	TWeakPtr<FSequenceValidationResult> WeakItem;
};

void SSequenceValidatorResultEntry::Construct(const FArguments& InArgs)
{
	WeakItem = InArgs._Item;

	TSharedRef<FSequenceValidatorStyle> ValidatorStyle = FSequenceValidatorStyle::Get();

	TSharedPtr<SHorizontalBox> ResultBox;

	TSharedPtr<FSequenceValidationResult> Item = WeakItem.Pin();

	if (Item && Item->GetChildren().Num() > 0)
	{
		if (!ResultBox)
		{
			ResultBox = SNew(SHorizontalBox);
		}

		// Add all the rules once at the parent level
		TSet<FString> RuleInfoNames;
		for (TSharedPtr<FSequenceValidationResult> Child : Item->GetChildren())
		{
			const FSequenceValidationRuleInfo& RuleInfo = Child->GetRuleInfo();
			if (RuleInfo.RuleName.IsEmpty() || RuleInfoNames.Contains(RuleInfo.RuleName.ToString()))
			{
				continue;
			}
			RuleInfoNames.Add(RuleInfo.RuleName.ToString());

			ResultBox->AddSlot()
			.Padding(FMargin(4, 0.f, 0.f, 0.f))
			.AutoWidth()
			[
				SNew(SImage)
				.Image(ValidatorStyle->GetBrush("ValidationRule.Rule"))
				.ColorAndOpacity_Lambda([RuleInfo]{ return RuleInfo.RuleColor; })
			];
		}

		if (RuleInfoNames.Num() == 0)
		{
			ResultBox->AddSlot()
			.Padding(FMargin(4, 0.f, 0.f, 0.f))
			.AutoWidth()
			[
				SNew(SImage)
				.Image(ValidatorStyle->GetBrush("ValidationResult.Pass"))
			];
		}
	}

	ChildSlot
	.Padding(0, 4, 5, 4)
	.VAlign(VAlign_Center)
	[
		SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.Padding(2, 2)
		[
			SNew(SImage)
			.Image(this, &SSequenceValidatorResultEntry::GetIcon)
			.ColorAndOpacity(this, &SSequenceValidatorResultEntry::GetColorAndOpacity)
		]
		+SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.Padding(2, 2)
		[
			SNew(STextBlock)
			.Text(this, &SSequenceValidatorResultEntry::GetMessage)
			.AutoWrapText(true)
		]
		+ SHorizontalBox::Slot()
		.HAlign(HAlign_Right)
		.VAlign(VAlign_Center)
		.Padding(2, 2)
		[
			ResultBox.IsValid()
				? ResultBox.ToSharedRef()
				: SNullWidget::NullWidget
		]
	];
}

const FSlateBrush* SSequenceValidatorResultEntry::GetIcon() const
{
	if (TSharedPtr<FSequenceValidationResult> Item = WeakItem.Pin())
	{
		if (UObject* Target = Item->GetTarget())
		{
			TSharedRef<FSequenceValidatorStyle> ValidatorStyle = FSequenceValidatorStyle::Get();

			if (Target->IsA<UMovieSceneSection>())
			{
				return ValidatorStyle->GetBrush("ValidationResult.Section");
			}
			else if (Target->IsA<UMovieSceneTrack>())
			{
				return ValidatorStyle->GetBrush("ValidationResult.Track");
			}
			else if (Target->IsA<UMovieSceneSequence>())
			{
				return ValidatorStyle->GetBrush("ValidationResult.Sequence");
			}
		}
	}
	return nullptr;
}

FSlateColor SSequenceValidatorResultEntry::GetColorAndOpacity() const
{
	if (TSharedPtr<FSequenceValidationResult> Item = WeakItem.Pin())
	{
		return Item->GetRuleInfo().RuleColor;
	}

	return FSlateColor::UseForeground();
}

FText SSequenceValidatorResultEntry::GetMessage() const
{
	if (TSharedPtr<FSequenceValidationResult> Item = WeakItem.Pin())
	{
		const FText& UserMessage = Item->GetUserMessage();
		if (!UserMessage.IsEmpty())
		{
			return UserMessage;
		}

		UObject* Target = Item->GetTarget();
		if (Target)
		{
			return FText::FromString(GetNameSafe(Target));
		}
	}
	return FText::GetEmpty();
}

void SSequenceValidatorResults::Construct(const FArguments& InArgs)
{
	Validator = InArgs._Validator;

	TSharedRef<FSequenceValidatorStyle> ValidatorStyle = FSequenceValidatorStyle::Get();

	ChildSlot
	[
		SNew(SVerticalBox)
		+SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SBorder)
			.Padding(0.f)
			.BorderImage(FAppStyle::Get().GetBrush("ToolPanel.GroupBorder"))
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(8.f, 4.f)
				.VAlign(VAlign_Center)
				[
					SNew(SImage)
					.Image(ValidatorStyle->GetBrush("SequenceValidator.ResultsTitleIcon"))
				]
				+ SHorizontalBox::Slot()
				.Padding(0.f, 4.f)
				.FillWidth(1.f)
				[
					SNew(STextBlock)
					.Margin(4.f)
					.Text(LOCTEXT("ResultsTitle", "Results"))
					.TextStyle(ValidatorStyle, "SequenceValidator.PanelTitleText")
				]
			]
		]
		+SVerticalBox::Slot()
		.FillHeight(1.f)
		[
			SNew(SOverlay)
			+SOverlay::Slot()
			[
				SNew(SBorder)
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.Visibility(this, &SSequenceValidatorResults::GetEmptyResultsMessageVisibility)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("EmptyResultsMessage", "All good!"))
				]
			]
			+SOverlay::Slot()
			[
				SAssignNew(TreeView, STreeView<FTreeItemPtr>)
				.TreeItemsSource(&ItemSource)
				.OnGenerateRow(this, &SSequenceValidatorResults::OnTreeViewGenerateRow)
				.OnGetChildren(this, &SSequenceValidatorResults::OnTreeViewGetChildren)
				.OnMouseButtonDoubleClick(this, &SSequenceValidatorResults::OnTreeViewMouseButtonDoubleClick)
			]
		]
	];
}

TSharedRef<ITableRow> SSequenceValidatorResults::OnTreeViewGenerateRow(TSharedPtr<FSequenceValidationResult> Item, const TSharedRef<STableViewBase>& OwnerTable)
{
	TSharedRef<FSequenceValidatorStyle> ValidatorStyle = FSequenceValidatorStyle::Get();
	return SNew(STableRow<TSharedPtr<FSequenceValidationResult>>, OwnerTable)
			.Style(&ValidatorStyle->GetWidgetStyle<FTableRowStyle>("ValidationResult.RowStyle"))
			[
				SNew(SSequenceValidatorResultEntry)
					.Item(Item)
			];
}

void SSequenceValidatorResults::OnTreeViewGetChildren(FTreeItemPtr Item, TArray<FTreeItemPtr>& OutChildren)
{
	OutChildren.Append(Item->GetChildren());
}

void SSequenceValidatorResults::OnTreeViewMouseButtonDoubleClick(FTreeItemPtr Item)
{
	TSharedPtr<FSequenceValidationResult> RootItem = Item->GetRoot();
	if (!RootItem)
	{
		return;
	}

	UObject* Asset = RootItem->GetTarget();
	if (!Asset)
	{
		return;
	}
	
	// Open the root sequence for this result, and grab the Sequencer editor that should now be open.
	UAssetEditorSubsystem* AssetEditorSubsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>();
	{
		FScopedSlowTask SlowTask(1.f, LOCTEXT("OpenSequenceSlowTask", "Loading sequence..."));
		SlowTask.MakeDialog();
		SlowTask.EnterProgressFrame();

		AssetEditorSubsystem->OpenEditorForAsset(Asset);
	}

	TSharedPtr<ISequencer> Sequencer;
	if (Asset->IsA<ULevelSequence>())
	{
		ILevelSequenceEditorToolkit* SequencerEditor = (ILevelSequenceEditorToolkit*)AssetEditorSubsystem->FindEditorForAsset(Asset, true);
		if (SequencerEditor)
		{
			Sequencer = SequencerEditor->GetSequencer();
		}
	}

	if (!ensureMsgf(Sequencer, TEXT("Couldn't open Sequencer editor for root sequence '%s'"), *GetNameSafe(Asset)))
	{
		return;
	}

	TSharedPtr<FSequencerSelection> Selection = Sequencer->GetViewModel()->GetSelection();

	const bool bAdd = FSlateApplication::Get().GetModifierKeys().IsShiftDown();
	if (!bAdd)
	{
		Selection->Empty();
	}

	UObject* Target = Item->GetTarget();
	if (Target)
	{
		// Focus the sub-sequence for the target (it might be a sub-sequence, or the currently open Sequencer
		// may be focused on something else).
		Sequencer->PopToSequenceInstance(MovieSceneSequenceID::Root);
		TArray<UMovieSceneSubSection*> SubSectionTrail;
		Item->GetSubSectionTrail(SubSectionTrail);
		for (UMovieSceneSubSection* SubSection : SubSectionTrail)
		{
			Sequencer->FocusSequenceInstance(*SubSection);
		}

		// Select the track/section/etc that relates to the message.
		if (UMovieSceneTrack* TargetTrack = Cast<UMovieSceneTrack>(Target))
		{
			Sequencer->SelectTrack(TargetTrack);
		}
		else if (UMovieSceneSection* TargetSection = Cast<UMovieSceneSection>(Target))
		{
			Sequencer->SelectSection(TargetSection);
		}

		FTargetKeys TargetKeys = Item->GetTargetKeys();
		if (TargetKeys.Channel != nullptr)
		{
			TSharedPtr<FChannelModel> ChannelModel;

			for (const FViewModelPtr TrackAreaItem : Selection->TrackArea)
			{
				if (const TViewModelPtr<FSectionModel> SectionModel = TrackAreaItem.ImplicitCast())
				{
					TParentFirstChildIterator<FChannelGroupModel> KeyAreaNodes = SectionModel->GetParentSectionOwnerModel().AsModel()->GetDescendantsOfType<FChannelGroupModel>();
					for (const TViewModelPtr<FChannelGroupModel>& KeyAreaNode : KeyAreaNodes)
					{
						TSharedPtr<FChannelModel> KeyAreaChannelModel = KeyAreaNode->GetChannel(SectionModel);
						if (KeyAreaChannelModel && KeyAreaChannelModel->GetChannel() == TargetKeys.Channel)
						{
							ChannelModel = KeyAreaChannelModel;
							break;
						}
					}
				}
				if (ChannelModel)
				{
					break;
				}
			}

			if (ChannelModel)
			{
				for (FKeyHandle KeyHandle : TargetKeys.KeyHandles)
				{
					Selection->KeySelection.Select(ChannelModel, KeyHandle);
				}
			}
		}

		// Move the time head to a time that relates to the message.
		if (Item->HasLocalTime())
		{
			Sequencer->SetLocalTime(Item->GetLocalTime());
		}
	}
}

EVisibility SSequenceValidatorResults::GetEmptyResultsMessageVisibility() const
{
	return ItemSource.IsEmpty() ? EVisibility::Visible : EVisibility::Hidden;
}

void SSequenceValidatorResults::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	SCompoundWidget::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);

	if (bUpdateItemSource)
	{
		bUpdateItemSource = false;

		UpdateItemSource();

		TreeView->RequestTreeRefresh();

		for (FTreeItemPtr Item : ItemSource)
		{
			// Collapse any items that fully passed validation
			bool bPassed = true;
			if (Item->GetChildren().Num() > 0)
			{
				for (TSharedPtr<FSequenceValidationResult> Child : Item->GetChildren())
				{
					const FSequenceValidationRuleInfo& RuleInfo = Child->GetRuleInfo();
					if (!RuleInfo.RuleName.IsEmpty())
					{
						bPassed = false;
						break;
					}
				}
			}

			TreeView->SetItemExpansion(Item, !bPassed);
		}
	}
}

void SSequenceValidatorResults::RequestListRefresh()
{
	bUpdateItemSource = true;
}

void SSequenceValidatorResults::UpdateItemSource()
{
	ItemSource.Reset();

	const FSequenceValidationResults& Results = Validator->GetResults();
	for (TSharedPtr<FSequenceValidationResult> Result : Results.GetResults())
	{
		ItemSource.Add(Result);
	}
}

}  // namespace UE::Sequencer

#undef LOCTEXT_NAMESPACE

