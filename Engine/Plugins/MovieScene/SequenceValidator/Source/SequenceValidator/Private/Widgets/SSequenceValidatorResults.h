// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/STreeView.h"

namespace UE::Sequencer
{

class FSequenceValidator;
class FSequenceValidationResult;

class SSequenceValidatorResults : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SSequenceValidatorResults)
	{}
		SLATE_ARGUMENT(TSharedPtr<FSequenceValidator>, Validator)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

	void RequestListRefresh();

protected:

	// SWidget interface.
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

private:

	using FTreeItemPtr = TSharedPtr<FSequenceValidationResult>;

	TSharedRef<ITableRow> OnTreeViewGenerateRow(FTreeItemPtr Item, const TSharedRef<STableViewBase>& OwnerTable);
	void OnTreeViewGetChildren(FTreeItemPtr Item, TArray<FTreeItemPtr>& OutChildren);
	void OnTreeViewMouseButtonDoubleClick(FTreeItemPtr Item);

	void UpdateItemSource();
	
	EVisibility GetEmptyResultsMessageVisibility() const;

private:

	TSharedPtr<FSequenceValidator> Validator;

	TSharedPtr<STreeView<FTreeItemPtr>> TreeView;
	TArray<FTreeItemPtr> ItemSource;

	bool bUpdateItemSource = false;
};

}  // namespace UE::Sequencer

