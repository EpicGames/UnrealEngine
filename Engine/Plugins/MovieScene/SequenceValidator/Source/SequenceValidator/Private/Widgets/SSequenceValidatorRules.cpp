// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SSequenceValidatorRules.h"

#include "SequenceValidatorStyle.h"
#include "Validation/SequenceValidationRule.h"
#include "Validation/SequenceValidator.h"
#include "Widgets/Input/SCheckBox.h"

#define LOCTEXT_NAMESPACE "SSequenceValidatorRules"

namespace UE::Sequencer
{

class SSequenceValidatorRuleEntry : public STableRow<TSharedPtr<FSequenceValidationRuleInfo>>
{
public:

	SLATE_BEGIN_ARGS(SSequenceValidatorRuleEntry)
	{}
		SLATE_ARGUMENT(TWeakPtr<FSequenceValidationRuleInfo>, Item)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& OwnerTable);

private:

	FText GetRuleName() const;
	FText GetRuleDescription() const;
	FSlateColor GetColorAndOpacity() const;
	ECheckBoxState IsRuleEnabled() const;
	void OnRuleEnabledChanged(ECheckBoxState CheckState) const;

private:

	TWeakPtr<FSequenceValidationRuleInfo> WeakItem;
};

void SSequenceValidatorRuleEntry::Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& OwnerTable)
{
	using FSuperRowType = STableRow<TSharedPtr<FSequenceValidationRuleInfo>>;

	WeakItem = InArgs._Item;

	const ISlateStyle& AppStyle = FAppStyle::Get();
	TSharedRef<const FSequenceValidatorStyle> ValidatorStyle = FSequenceValidatorStyle::Get();

	ChildSlot
	.Padding(4.f)
	[
		SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		[
			SNew(SImage)
			.Image(ValidatorStyle->GetBrush("ValidationRule.Rule"))
			.ColorAndOpacity(this, &SSequenceValidatorRuleEntry::GetColorAndOpacity)
		]
		+SHorizontalBox::Slot()
		.Padding(8.f, 4.f)
		.FillWidth(1.f)
		[
			SNew(SVerticalBox)
			+SVerticalBox::Slot()
			.Padding(2.f, 4.f)
			[
				SNew(STextBlock)
				.Text(this, &SSequenceValidatorRuleEntry::GetRuleName)
				.TextStyle(ValidatorStyle, "ValidatorRule.RuleNameText")
				.ToolTipText(this, &SSequenceValidatorRuleEntry::GetRuleDescription)
			]
			+SVerticalBox::Slot()
			.Padding(2.f, 4.f)
			.AutoHeight()
			[
				SNew(STextBlock)
				.Text(this, &SSequenceValidatorRuleEntry::GetRuleDescription)
				.TextStyle(ValidatorStyle, "ValidatorRule.RuleDescriptionText")
				.AutoWrapText(true)
			]
		]
		+SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew(SCheckBox)
			.IsChecked(this, &SSequenceValidatorRuleEntry::IsRuleEnabled)
			.OnCheckStateChanged(this, &SSequenceValidatorRuleEntry::OnRuleEnabledChanged)
			.ToolTipText(LOCTEXT("ToggleValidationRule", "Toggle this validation rule"))
		]
	];

	FSuperRowType::ConstructInternal(
		FSuperRowType::FArguments()
			.Style(&ValidatorStyle->GetWidgetStyle<FTableRowStyle>("ValidationRule.RowStyle")),
		OwnerTable);
}

FText SSequenceValidatorRuleEntry::GetRuleName() const
{
	if (TSharedPtr<FSequenceValidationRuleInfo> Item = WeakItem.Pin())
	{
		return Item->RuleName;
	}
	return FText::GetEmpty();
}

FText SSequenceValidatorRuleEntry::GetRuleDescription() const
{
	if (TSharedPtr<FSequenceValidationRuleInfo> Item = WeakItem.Pin())
	{
		return Item->RuleDescription;
	}
	return FText::GetEmpty();
}

FSlateColor SSequenceValidatorRuleEntry::GetColorAndOpacity() const
{
	if (TSharedPtr<FSequenceValidationRuleInfo> Item = WeakItem.Pin())
	{
		return Item->RuleColor;
	}
	return FSlateColor::UseForeground();
}

ECheckBoxState SSequenceValidatorRuleEntry::IsRuleEnabled() const
{
	if (TSharedPtr<FSequenceValidationRuleInfo> Item = WeakItem.Pin())
	{
		return Item->bIsEnabled ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	}
	return ECheckBoxState::Undetermined;
}

void SSequenceValidatorRuleEntry::OnRuleEnabledChanged(ECheckBoxState CheckState) const
{
	if (TSharedPtr<FSequenceValidationRuleInfo> Item = WeakItem.Pin())
	{
		Item->bIsEnabled = (CheckState == ECheckBoxState::Checked);
	}
}

void SSequenceValidatorRules::Construct(const FArguments& InArgs)
{
	Validator = InArgs._Validator;

	TSharedRef<FSequenceValidatorStyle> ValidatorStyle = FSequenceValidatorStyle::Get();

	ChildSlot
	[
		SNew(SVerticalBox)
		+SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SBorder)
			.Padding(0.f)
			.BorderImage(FAppStyle::Get().GetBrush("ToolPanel.GroupBorder"))
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(8.f, 4.f)
				.VAlign(VAlign_Center)
				[
					SNew(SImage)
					.Image(ValidatorStyle->GetBrush("SequenceValidator.RulesTitleIcon"))
				]
				+ SHorizontalBox::Slot()
				.Padding(0.f, 4.f)
				.FillWidth(1.f)
				[
					SNew(STextBlock)
					.Margin(4.f)
					.Text(LOCTEXT("RulesTitle", "Rules"))
					.TextStyle(ValidatorStyle, "SequenceValidator.PanelTitleText")
				]
			]
		]
		+SVerticalBox::Slot()
		.FillHeight(1.f)
		[
			SAssignNew(ListView, SListView<FListItemPtr>)
			.ListItemsSource(&ItemSource)
			.OnGenerateRow(this, &SSequenceValidatorRules::OnListGenerateItemRow)
		]
	];

	UpdateItemSource();
}

TSharedRef<ITableRow> SSequenceValidatorRules::OnListGenerateItemRow(FListItemPtr Item, const TSharedRef<STableViewBase>& OwnerTable)
{
	return SNew(SSequenceValidatorRuleEntry, OwnerTable)
		.Item(Item);
}

void SSequenceValidatorRules::UpdateItemSource()
{
	ItemSource.Reset();

	for (TSharedPtr<FSequenceValidationRuleInfo> RuleInfo : Validator->GetRules())
	{
		if (RuleInfo)
		{
			ItemSource.Add(RuleInfo);
		}
	}
}

}  // namespace UE::Sequencer

#undef LOCTEXT_NAMESPACE

