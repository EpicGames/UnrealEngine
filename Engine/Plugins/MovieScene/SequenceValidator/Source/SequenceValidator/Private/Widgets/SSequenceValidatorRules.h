// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/SListView.h"

namespace UE::Sequencer
{

class FSequenceValidator;
struct FSequenceValidationRuleInfo;

class SSequenceValidatorRules : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SSequenceValidatorRules)
	{}
		SLATE_ARGUMENT(TSharedPtr<const FSequenceValidator>, Validator)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

private:

	using FListItemPtr = TSharedPtr<FSequenceValidationRuleInfo>;

	TSharedRef<ITableRow> OnListGenerateItemRow(FListItemPtr Item, const TSharedRef<STableViewBase>& OwnerTable);

	void UpdateItemSource();

private:

	TSharedPtr<const FSequenceValidator> Validator;

	TSharedPtr<SListView<FListItemPtr>> ListView;
	TArray<FListItemPtr> ItemSource;
};

}  // namespace UE::Sequencer

