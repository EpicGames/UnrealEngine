// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"
#include "Logging/LogMacros.h"
#include "Modules/ModuleInterface.h"

DECLARE_LOG_CATEGORY_EXTERN(LogSequenceValidator, Log, All);

namespace UE::Sequencer
{

class FSequenceValidationRule;
struct FSequenceValidationRuleInfo;

using FSequenceValidationRuleID = FDelegateHandle;

/**
 * Interface for the Sequence Validator module.
 */
class ISequenceValidatorModule : public IModuleInterface
{
public:

	/** Registers a new validation rule factory. */
	virtual FSequenceValidationRuleID RegisterValidationRule(FSequenceValidationRuleInfo&& InRuleInfo) = 0;
	/** Gets the registered validation rule factories. */
	virtual TConstArrayView<FSequenceValidationRuleInfo> GetValidationRules() const = 0;
	/** Unregisters an existing validation rule factory. */
	virtual void UnregisterValidationRule(FSequenceValidationRuleID InRuleID) = 0;
};

}  // namespace UE::Sequencer

