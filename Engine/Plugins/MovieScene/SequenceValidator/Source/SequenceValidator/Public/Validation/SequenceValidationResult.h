// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Internationalization/Text.h"
#include "Logging/TokenizedMessage.h"
#include "Misc/FrameTime.h"
#include "Misc/Optional.h"
#include "SequenceValidationRule.h"
#include "Templates/SharedPointer.h"
#include "UObject/WeakObjectPtr.h"
#include "Channels/MovieSceneChannel.h"

class UMovieSceneSubSection;
class UMovieSceneSection;

namespace UE::Sequencer
{

struct FTargetKeys
{
	FMovieSceneChannel* Channel = nullptr;
	TArray<FKeyHandle> KeyHandles;
};

/**
 * A single validation result, indicating a note, warning, error, etc. inside the sequence.
 */
class FSequenceValidationResult : public TSharedFromThis<FSequenceValidationResult>
{
public:

	/** Creates a new default validation result. */
	FSequenceValidationResult() = default;

	/** Creates a new validation result for the given object. */
	FSequenceValidationResult(UObject* InTarget)
		: WeakTarget(InTarget)
	{}

	/** Creates a new validation result for the given object, with the given severity. */
	FSequenceValidationResult(EMessageSeverity::Type InSeverity, UObject* InTarget, const FSequenceValidationRuleInfo& InRuleInfo)
		: Severity(InSeverity)
		, WeakTarget(InTarget)
		, RuleInfo(InRuleInfo)
	{}

	/** Creates a new validation result with the given severity and message. */
	FSequenceValidationResult(EMessageSeverity::Type InSeverity, const FText& InUserMessage)
		: Severity(InSeverity)
		, UserMessage(InUserMessage)
	{}

	/** Creates a new validation result with the given severity and message, for the given object. */
	FSequenceValidationResult(EMessageSeverity::Type InSeverity, UObject* InTarget, const FText& InUserMessage, const FSequenceValidationRuleInfo& InRuleInfo)
		: Severity(InSeverity)
		, WeakTarget(InTarget)
		, UserMessage(InUserMessage)
		, RuleInfo(InRuleInfo)
	{}

public:

	EMessageSeverity::Type GetSeverity() const { return Severity; }
	const FSequenceValidationRuleInfo& GetRuleInfo() const { return RuleInfo; }

	SEQUENCEVALIDATOR_API bool GetSubSectionTrail(TArray<UMovieSceneSubSection*>& OutTrail) const;
	SEQUENCEVALIDATOR_API void SetSubSectionTrail(TArrayView<UMovieSceneSubSection*> InTrail);

	UObject* GetTarget() const { return WeakTarget.Get(); }
	void SetTarget(UObject* InTarget) { WeakTarget = InTarget; }

	FTargetKeys GetTargetKeys() const { return TargetKeys; }
	void SetTargetKeys(FTargetKeys InTargetKeys) { TargetKeys = InTargetKeys; }

	const FText& GetUserMessage() const { return UserMessage; }
	void SetUserMessage(const FText& InUserMessage) { UserMessage = InUserMessage; }

	bool HasLocalTime() const { return LocalTime.IsSet(); }
	FFrameTime GetLocalTime() const { return LocalTime.Get(FFrameTime(0)); }
	void SetLocalTime(const FFrameTime InLocalTime) { LocalTime = InLocalTime; }

public:

	TSharedPtr<FSequenceValidationResult> GetParent() const
	{
		return WeakParent.Pin();
	}

	TSharedPtr<FSequenceValidationResult> GetRoot();

	bool HasChildren() const { return !Children.IsEmpty(); }

	TArrayView<const TSharedPtr<FSequenceValidationResult>> GetChildren() const
	{
		return Children;
	}

	SEQUENCEVALIDATOR_API void AddChild(TSharedRef<FSequenceValidationResult> InChild);

	SEQUENCEVALIDATOR_API void AppendChildren(TConstArrayView<TSharedPtr<FSequenceValidationResult>> InChildren);

private:

	TWeakPtr<FSequenceValidationResult> WeakParent;
	TArray<TSharedPtr<FSequenceValidationResult>> Children;

	EMessageSeverity::Type Severity = EMessageSeverity::Info;
	TArray<TWeakObjectPtr<UMovieSceneSubSection>> WeakSubSectionTrail;
	TWeakObjectPtr<> WeakTarget;
	FTargetKeys TargetKeys;
	FText UserMessage;
	TOptional<FFrameTime> LocalTime;
	FSequenceValidationRuleInfo RuleInfo;
};

/**
 * A collection of validation results.
 */
class FSequenceValidationResults : public TSharedFromThis<FSequenceValidationResult>
{
public:

	SEQUENCEVALIDATOR_API void AddResult(TSharedRef<FSequenceValidationResult> InResult);
	SEQUENCEVALIDATOR_API void AppendResults(const FSequenceValidationResults& InResults);
	SEQUENCEVALIDATOR_API void AppendResults(TConstArrayView<TSharedPtr<FSequenceValidationResult>> InResults);

	SEQUENCEVALIDATOR_API TArrayView<const TSharedPtr<FSequenceValidationResult>> GetResults() const;

	SEQUENCEVALIDATOR_API void Reset();

private:

	TArray<TSharedPtr<FSequenceValidationResult>> ValidationResults;
};

}  // namespace UE::Sequencer

