// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Internationalization/Text.h"
#include "Styling/SlateColor.h"
#include "Templates/SharedPointer.h"

class UMovieSceneSequence;

namespace UE::Sequencer
{

class FSequenceValidationResults;
class FSequenceValidationRule;

DECLARE_DELEGATE_RetVal(TSharedRef<FSequenceValidationRule>, FOnCreateSequenceValidationRule);

/** Basic information regarding a validation rule, and how to create it. */
struct FSequenceValidationRuleInfo
{
	/** The name of the rule. */
	FText RuleName;
	/** The description of the rule. */
	FText RuleDescription;
	/** A factory method for creating an instance of the rule. */
	FOnCreateSequenceValidationRule RuleFactory;
	/** Whether this rule is enabled. */
	bool bIsEnabled = true;
	/** Color to display in the UI */
	FSlateColor RuleColor = FSlateColor::UseForeground();
};


/**
 * Base class for a validation rule that can be run on a sequence.
 */
class FSequenceValidationRule : public TSharedFromThis<FSequenceValidationRule>
{
public:

	FSequenceValidationRule() = default;
	virtual ~FSequenceValidationRule() = default;

public:

	void Run(const UMovieSceneSequence* InSequence, FSequenceValidationResults& OutResults) const;

	// Get the color for the rule from an internal palette
	static FSlateColor GetRuleColor(const FText& InRuleName);

protected:

	virtual void OnRun(const UMovieSceneSequence* InSequence, FSequenceValidationResults& OutResults) const {}
};

}  // namespace UE::Sequencer

