// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Delegates/Delegate.h"
#include "ISequenceValidatorModule.h"
#include "Templates/SharedPointerFwd.h"
#include "Templates/UniquePtr.h"
#include "Validation/SequenceValidationResult.h"

class UMovieSceneSequence;

namespace UE::Sequencer
{

class FSequenceValidationRule;

namespace Internal { class FSequenceValidatorTaskScheduler; }

/**
 * A sequence validator that can run a given set of validation rules on a sequence.
 *
 * Validation rules are created along with the validator instance, using whatever validation
 * rules factories are registered on the ISequenceValidatorModule at the time.
 */
class FSequenceValidator
{
public:

	FSequenceValidator();
	~FSequenceValidator();

public:

	/** The current queue of sequences to validate. */
	TArrayView<UMovieSceneSequence* const> GetQueue() const;
	/** The list of validation rules to run on the queue. */
	TArray<TSharedPtr<FSequenceValidationRuleInfo>> GetRules() const;
	/** The current list of results generated by the validation rules. */
	const FSequenceValidationResults& GetResults() const;

	/** Add the given root sequence to the queue. */
	void Queue(UMovieSceneSequence* InSequence);
	/** Add the given root sequences to the queue. */
	void Queue(TArrayView<UMovieSceneSequence*> InSequences);
	/** Remove a sequence from the queue. */
	void Delete(UMovieSceneSequence* InSequence);
	/** Clears the current queue. */
	void ClearQueue();

	/**
	 * Start validating the current queue.
	 * This runs all validation rules on all queued-up sequences in an asynchronous manner.
	 * Call the IsValidating() method to know when the validation has ended.
	 */
	void StartValidation();

	/** Whether an asynchronous validation is in progress. */
	bool IsValidating() const;

	/** Gets an event that will be triggered when an asynchronous validation ends. */
	FSimpleMulticastDelegate& GetOnValidationFinished();

public:

	/** Validates the given sequence synchronously. */
	void Validate(UMovieSceneSequence* InSequence);
	/** Validates the given sequences synchronously. */
	void Validate(TArrayView<UMovieSceneSequence*> InSequences);

private:

	void CreateValidationRuleInfos();
	void StartValidation(bool bAsync);
	void OnValidationFinished();

private:

	TArray<UMovieSceneSequence*> ValidationQueue;
	TArray<TSharedRef<FSequenceValidationRuleInfo>> ValidationRuleInfos;
	FSequenceValidationResults ValidationResults;

	TUniquePtr<Internal::FSequenceValidatorTaskScheduler> Scheduler;

	FSimpleMulticastDelegate OnValidationFinishedDelegate;
};

}  // namespace UE::Sequencer

