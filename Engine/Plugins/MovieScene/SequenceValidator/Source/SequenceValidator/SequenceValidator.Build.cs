// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class SequenceValidator : ModuleRules
{
	public SequenceValidator(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"ApplicationCore",
				"AssetRegistry",
				"ContentBrowser",
				"Core",
				"CoreUObject",
				"Engine",
				"InputCore",
				"LevelSequence",
				"LevelSequenceEditor",
				"MovieScene",
				"MovieSceneTracks",
				"Projects",
				"Sequencer",
				"SequencerCore",
				"Slate",
				"SlateCore",
				"ToolMenus",
				"ToolWidgets",
				"UnrealEd",
				"WorkspaceMenuStructure",
			}
		);
	}
}

